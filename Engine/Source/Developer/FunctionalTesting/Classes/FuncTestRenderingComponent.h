// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "FuncTestRenderingComponent.generated.h"

class FPrimitiveSceneProxy;

UCLASS(hidecategories=Object, editinlinenew)
class UFuncTestRenderingComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	//~ Begin UPrimitiveComponent Interface
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual bool ShouldRecreateProxyOnUpdateTransform() const override { return true; }
	//~ End UPrimitiveComponent Interface

	//~ Begin USceneComponent Interface
	virtual FBoxSphereBounds CalcBounds(const FTransform &LocalToWorld) const override;
	//~ End USceneComponent Interface
};
