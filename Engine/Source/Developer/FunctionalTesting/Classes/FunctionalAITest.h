// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Pawn.h"
#include "FunctionalTest.h"
#include "GenericTeamAgentInterface.h"

#include "FunctionalAITest.generated.h"

#define UE_API FUNCTIONALTESTING_API

class AAIController;
class AFunctionalAITest;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FFunctionalTestAISpawned, AAIController*, Controller, APawn*, Pawn);

/**
*	FAITestSpawnInfoBase
*
*	Base struct defining where & when to spawn. Used within a FAITestSpawnSetBase class.
*/
USTRUCT(BlueprintType)
struct FAITestSpawnInfoBase
{
	GENERATED_USTRUCT_BODY()

	/** Where should AI be spawned */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AISpawn)
	TObjectPtr<AActor> SpawnLocation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AISpawn, meta=(UIMin=1, ClampMin=1))
	int32 NumberToSpawn;

	/** delay between consecutive spawn attempts */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AISpawn, meta = (UIMin = 0, ClampMin = 0))
	float SpawnDelay;

	/** delay before attempting first spawn */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AISpawn, meta = (UIMin = 0, ClampMin = 0))
	float PreSpawnDelay;

	/** Gets filled owning spawn set upon game start */
	FName SpawnSetName;

	FAITestSpawnInfoBase()
		: SpawnLocation(nullptr)
		, NumberToSpawn(1)
		, SpawnDelay(0.0f)
		, PreSpawnDelay(0.0f)
	{}

	virtual ~FAITestSpawnInfoBase() = default;

	virtual bool IsValid() const { return SpawnLocation != NULL; }

	virtual bool Spawn(AFunctionalAITestBase* AITest) const PURE_VIRTUAL(, return false;);
};

template<>
struct TStructOpsTypeTraits<FAITestSpawnInfoBase> : public TStructOpsTypeTraitsBase2<FAITestSpawnInfoBase>
{
	enum
	{
		WithPureVirtual = true,
	};
};

/**
*	FAITestSpawnInfo
*
*	Generic AI Test Spawn Info used in FAITestSpawnSet within a generic AFunctionalAITest test.
*/
USTRUCT(BlueprintType)
struct FAITestSpawnInfo : public FAITestSpawnInfoBase
{
	GENERATED_BODY()

	/** Determines AI to be spawned */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AISpawn)
	TSubclassOf<class APawn>  PawnClass;
	
	/** class to override default pawn's controller class. If None the default will be used*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AISpawn)
	TSubclassOf<class AAIController>  ControllerClass;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AISpawn)
	FGenericTeamId TeamID;

	/** if set will be applied to spawned AI */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AISpawn)
	TObjectPtr<class UBehaviorTree> BehaviorTree;

	FAITestSpawnInfo()
		: BehaviorTree(nullptr)
	{}

	FORCEINLINE virtual bool IsValid() const override { return PawnClass != NULL && FAITestSpawnInfoBase::IsValid(); }

	UE_API virtual bool Spawn(AFunctionalAITestBase* AITest) const override;
};


/**
*	FPendingDelayedSpawn
*
*	Struct defining a pending spawn request within a AFunctionalAITestBase.
*/
USTRUCT(BlueprintType)
struct FPendingDelayedSpawn
{
	GENERATED_BODY()

	uint32 NumberToSpawnLeft;
	float TimeToNextSpawn;
	bool bFinished;

	/** Index to spawn in the SpawnSets */
	uint32 SpawnSetIndex;

	/** Index to spawn in the SpawnInfoContainer's spawnset  */
	uint32 SpawnInfoIndex;

	FPendingDelayedSpawn()
		: NumberToSpawnLeft(uint32(-1))
		, TimeToNextSpawn(FLT_MAX)
		, bFinished(true)
		, SpawnSetIndex(uint32(-1))
		, SpawnInfoIndex(uint32(-1))
	{}

	FPendingDelayedSpawn(const uint32 InSpawnSetIndex, const uint32 InSpawnInfoIndex, const int32 InNumberToSpawnLeft, const float InTimeToNextSpawn)
		: NumberToSpawnLeft(InNumberToSpawnLeft)
		, TimeToNextSpawn(InTimeToNextSpawn)
		, bFinished(false)
		, SpawnSetIndex(InSpawnSetIndex)
		, SpawnInfoIndex(InSpawnInfoIndex)
	{
	}

	void Tick(float TimeDelta, AFunctionalAITestBase* AITest);
};

/**
*	FAITestSpawnSetBase
*
*	Base struct defining an AI Test Spawn Set that are used in AFunctionalAITestBase tests.
*/
USTRUCT(BlueprintType)
struct FAITestSpawnSetBase
{
	GENERATED_BODY()

	/** give the set a name to help identify it if need be */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AISpawn)
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AISpawn)
	uint32 bEnabled:1;

	/** location used for spawning if spawn info doesn't define one */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AISpawn)
	TObjectPtr<AActor> FallbackSpawnLocation;

	FAITestSpawnSetBase()
		: bEnabled(true)
		, FallbackSpawnLocation(nullptr)
	{}

	virtual ~FAITestSpawnSetBase() = default;

	/** Return the FAITestSpawnInfoBase at this index of the SpawnInfoContainer array. Const-correct version. */
	virtual const FAITestSpawnInfoBase* GetSpawnInfo(const int32 SpawnInfoIndex) const PURE_VIRTUAL(, return nullptr;);

	/** Return the FAITestSpawnInfoBase at this index of the SpawnInfoContainer array. */
	virtual FAITestSpawnInfoBase* GetSpawnInfo(const int32 SpawnInfoIndex) PURE_VIRTUAL(, return nullptr;);

	/** Return whether the index is valid in the SpawnInfoContainer array. */
	virtual bool IsValidSpawnInfoIndex(const int32 Index) const PURE_VIRTUAL(, return false;);

	/** Pure virtual method to iterate through the spawn info container and execute Predicate on each in a const-correct way. */
	virtual void ForEachSpawnInfo(TFunctionRef<void(const FAITestSpawnInfoBase&)> Predicate) const PURE_VIRTUAL(, );
	
	/** Pure virtual method to iterate through the spawn info container and execute Predicate on each. */
	virtual void ForEachSpawnInfo(TFunctionRef<void(FAITestSpawnInfoBase&)> Predicate) PURE_VIRTUAL(,);
};
template<>
struct TStructOpsTypeTraits<FAITestSpawnSetBase> : public TStructOpsTypeTraitsBase2<FAITestSpawnSetBase>
{
	enum
	{
		WithPureVirtual = true,
	};
};

/** 
*	FAITestSpawnSet
*
*	Generic AI Test Spawn Set that is used in regular AFunctionalAITest tests.
*/
USTRUCT(BlueprintType)
struct FAITestSpawnSet : public FAITestSpawnSetBase
{
	GENERATED_BODY()

	FAITestSpawnSet() {}

	/** To iterate through the spawn info container and execute Predicate on each in a const-correct way. */
	virtual void ForEachSpawnInfo(TFunctionRef<void(const FAITestSpawnInfoBase&)> Predicate) const override;

	/** To iterate through the spawn info container and execute Predicate on each. */
	virtual void ForEachSpawnInfo(TFunctionRef<void(FAITestSpawnInfoBase&)> Predicate) override;

	/** Return the FAITestSpawnInfoBase at this index of the SpawnInfoContainer array. Const-correct version. */
	virtual const FAITestSpawnInfoBase* GetSpawnInfo(const int32 SpawnInfoIndex) const override;

	/** Return the FAITestSpawnInfoBase at this index of the SpawnInfoContainer array. */
	virtual FAITestSpawnInfoBase* GetSpawnInfo(const int32 SpawnInfoIndex) override;

	/** Return whether the index is valid in the SpawnInfoContainer array. */
	virtual bool IsValidSpawnInfoIndex(const int32 Index) const override;
	
protected:
	/** what to spawn */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AISpawn)
	TArray<FAITestSpawnInfo> SpawnInfoContainer;
};

/** 
*	AFunctionalAITestBase
*
*	Base abstract class defining a Functional AI Test.
*	You can derive from this base class to create a test with a different type of SpawnSets.
*/
UCLASS(MinimalAPI, Abstract, BlueprintType)
class AFunctionalAITestBase : public AFunctionalTest
{
	GENERATED_BODY()

public:
	UE_API AFunctionalAITestBase(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Iterate through the list of spawn sets and execute Predicate on each in a const-correct way. */
	virtual void ForEachSpawnSet(TFunctionRef<void(const FAITestSpawnSetBase&)> Predicate) const PURE_VIRTUAL(, );

	/** Iterate through the list of spawn sets and execute Predicate on each. */
	virtual void ForEachSpawnSet(TFunctionRef<void(FAITestSpawnSetBase&)> Predicate) PURE_VIRTUAL(, );

	/** Iterate through the list of spawn sets and remove the spawn set if Predicate returns true. */
	virtual void RemoveSpawnSetIfPredicate(TFunctionRef<bool(FAITestSpawnSetBase&)> Predicate) PURE_VIRTUAL(, );

	/** Return the SpawnSet at this index of the SpawnSets array. Const-correct version. */
	virtual const FAITestSpawnSetBase* GetSpawnSet(const int32 SpawnSetIndex) const PURE_VIRTUAL(, return nullptr;);

	/** Return the SpawnSet at this index of the SpawnSets array. */
	virtual FAITestSpawnSetBase* GetSpawnSet(const int32 SpawnSetIndex) PURE_VIRTUAL(,return nullptr;);
	
	/** Return the SpawnInfo at SpawnInfoIndexof the SpawnSet at SpawnSetIndex. Const-correct version. */
	UE_API const FAITestSpawnInfoBase* GetSpawnInfo(const int32 SpawnSetIndex, const int32 SpawnInfoIndex) const;

	/** Return the SpawnInfo at SpawnInfoIndexof the SpawnSet at SpawnSetIndex. */
	UE_API FAITestSpawnInfoBase* GetSpawnInfo(const int32 SpawnSetIndex, const int32 SpawnInfoIndex);

	/** Return whether the index is valid in the SpawnSets array. */
	virtual bool IsValidSpawnSetIndex(const int32 Index) const PURE_VIRTUAL(, return false;);

	/** 
	* Spawn this AI at this SpawnInfoIndex of the SpawnSetIndex spawn set. 
	*
	* @param SpawnSetIndex	The index of the spawn set in the SpawnSets array
	* @param SpawnInfoIndex	The index of the spawn info in the spawn set
	*
	* @return True if spawn was successful.
	*/
	UE_API bool Spawn(const int32 SpawnSetIndex, const int32 SpawnInfoIndex);
protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AITest, meta = (UIMin = "0.0"))
	float SpawnLocationRandomizationRange;

	UPROPERTY(BlueprintReadOnly, Category=AITest)
	TArray<TObjectPtr<APawn>> SpawnedPawns;

	UPROPERTY(BlueprintReadOnly, Category = AITest)
	TArray<FPendingDelayedSpawn> PendingDelayedSpawns;
	
	UPROPERTY(BlueprintReadOnly, Category = AITest)
	int32 CurrentSpawnSetIndex;

	UPROPERTY(BlueprintReadOnly, Category = AITest)
	FString CurrentSpawnSetName;

	/** Called when a single AI finished spawning */
	UPROPERTY(BlueprintAssignable)
	FFunctionalTestAISpawned OnAISpawned;

	/** Called when a all AI finished spawning */
	UPROPERTY(BlueprintAssignable)
	FFunctionalTestEventSignature OnAllAISPawned;

	/** navmesh debug: log navoctree modifiers around this point */
	UPROPERTY(EditAnywhere, Category = NavMeshDebug, meta = (MakeEditWidget = ""))
	FVector NavMeshDebugOrigin;

	/** navmesh debug: extent around NavMeshDebugOrigin */
	UPROPERTY(EditAnywhere, Category = NavMeshDebug)
	FVector NavMeshDebugExtent;

	/** if set, ftest will postpone start until navmesh is fully generated */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AITest)
	uint32 bWaitForNavMesh : 1;

	/** if set, ftest will postpone start until navmesh is fully generated */
	UPROPERTY(EditAnywhere, Category = NavMeshDebug)
	uint32 bDebugNavMeshOnTimeout : 1;

	uint32 bSingleSetRun:1;

public:
	UFUNCTION(BlueprintCallable, Category = "Development")
	UE_API virtual bool IsOneOfSpawnedPawns(AActor* Actor);

	// AActor interface begin
protected:
	UE_API virtual void BeginPlay() override;

public:
	UE_API virtual void Tick(float DeltaSeconds) override;
	// AActor interface end

	UE_API virtual bool RunTest(const TArray<FString>& Params = TArray<FString>()) override;
	UE_API virtual void StartTest() override;
	UE_API virtual void OnTimeout() override;
	UE_API virtual bool IsReady_Implementation() override;
	UE_API virtual bool WantsToRunAgain() const override;
	UE_API virtual void GatherRelevantActors(TArray<AActor*>& OutActors) const override;
	UE_API virtual void CleanUp() override;
	UE_API virtual FString GetAdditionalTestFinishedMessage(EFunctionalTestResult TestResult) const override;
	UE_API virtual FString GetReproString() const override;

	UE_API void AddSpawnedPawn(APawn& SpawnedPawn);

	UE_API FVector GetRandomizedLocation(const FVector& Location) const;

protected:

	UE_API void KillOffSpawnedPawns();
	UE_API void ClearPendingDelayedSpawns();
	UE_API void StartSpawning();
	UE_API void OnSpawningFailure();
	UE_API bool IsNavMeshReady() const;

	FTimerHandle NavmeshDelayTimer;
};

/** 
*	FuntionalAITest
*
*	Functional AI Test using a regular FAITestSpawnSet as a default SpawnSet class type.
*/
UCLASS(MinimalAPI, Blueprintable)
class AFunctionalAITest : public AFunctionalAITestBase
{
	GENERATED_BODY()

public:
	/** Iterate through the list of spawn sets and execute Predicate on each in a const-correct way. */
	UE_API virtual void ForEachSpawnSet(TFunctionRef<void(const FAITestSpawnSetBase&)> Predicate) const override;

	/** Iterate through the list of spawn sets and execute Predicate on each. */
	UE_API virtual void ForEachSpawnSet(TFunctionRef<void(FAITestSpawnSetBase&)> Predicate) override;

	/** Iterate through the list of spawn sets and remove the spawn set if Predicate returns true. */
	UE_API virtual void RemoveSpawnSetIfPredicate(TFunctionRef<bool(FAITestSpawnSetBase&)> Predicate) override;

	/** Return the SpawnSet at this index of the SpawnSets array. Const-correct version. */
	UE_API virtual const FAITestSpawnSetBase* GetSpawnSet(const int32 SpawnSetIndex) const override;

	/** Return the SpawnSet at this index of the SpawnSets array. */
	UE_API virtual FAITestSpawnSetBase* GetSpawnSet(const int32 SpawnSetIndex) override;

	/** Return whether the index is valid in the SpawnSets array. */
	UE_API virtual bool IsValidSpawnSetIndex(const int32 Index) const override;

protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AITest)
	TArray<FAITestSpawnSet> SpawnSets;
};

#undef UE_API
