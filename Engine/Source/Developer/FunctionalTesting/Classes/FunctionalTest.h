// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Math/RandomStream.h"
#include "Stats/Stats.h"
#include "GameFramework/Actor.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "Math/StatisticalFloat.h"
#include "Tests/AutomationCommon.h"
#include "FunctionalTest.generated.h"

#define UE_API FUNCTIONALTESTING_API

class Error;
class UBillboardComponent;
class UTraceQueryTestResults;
class UWorld;

DECLARE_STATS_GROUP(TEXT("FunctionalTest"), STATGROUP_FunctionalTest, STATCAT_Advanced);

#if UE_EXTERNAL_PROFILING_ENABLED
//Experimental effort at automated cpu captures from the functional testing.
class FFunctionalTestExternalProfiler : public FScopedExternalProfilerBase
{
public:
	void StartProfiler(const bool bWantPause){ StartScopedTimer(bWantPause); }
	void StopProfiler(){StopScopedTimer();}
}; 
#endif	// UE_EXTERNAL_PROFILING_ENABLED

struct FStatsData
{
	FStatsData() :NumFrames(0), SumTimeSeconds(0.0f){}

	uint32 NumFrames;
	float SumTimeSeconds;
	FStatisticalFloat FrameTimeTracker;
	FStatisticalFloat GameThreadTimeTracker;
	FStatisticalFloat RenderThreadTimeTracker;
	FStatisticalFloat GPUTimeTracker;
};

/** A set of simple perf stats recorded over a period of frames. */
struct FPerfStatsRecord
{
	UE_API FPerfStatsRecord(FString InName);

	FString Name;

	/** Stats data for the period we're interested in timing. */
	FStatsData Record;
	/** Stats data for the baseline. */
	FStatsData Baseline;
	
	float GPUBudget;
	float RenderThreadBudget;
	float GameThreadBudget;

	UE_API void SetBudgets(float InGPUBudget, float InRenderThreadBudget, float InGameThreadBudget);
	UE_API void Sample(UWorld* Owner, float DeltaSeconds, bool bBaseline);

	UE_API FString GetReportString()const;
	UE_API FString GetBaselineString()const;
	UE_API FString GetRecordString()const;
	UE_API FString GetOverBudgetString()const;

	UE_API void GetGPUTimes(double& OutMin, double& OutMax, double& OutAvg)const;
	UE_API void GetGameThreadTimes(double& OutMin, double& OutMax, double& OutAvg)const;
	UE_API void GetRenderThreadTimes(double& OutMin, double& OutMax, double& OutAvg)const;

	UE_API bool IsWithinGPUBudget()const;
	UE_API bool IsWithinGameThreadBudget()const;
	UE_API bool IsWithinRenderThreadBudget()const;
};

UENUM(BlueprintType)
enum class EComparisonMethod : uint8
{
	Equal_To,
	Not_Equal_To,
	Greater_Than_Or_Equal_To,
	Less_Than_Or_Equal_To,
	Greater_Than,
	Less_Than
};


/** 
 * Class for use with functional tests which provides various performance measuring features. 
 * Recording of basic, unintrusive performance stats.
 * Automatic captures using external CPU and GPU profilers.
 * Triggering and ending of writing full stats to a file.
*/
UCLASS(MinimalAPI, Blueprintable)
class UAutomationPerformaceHelper : public UObject
{
	GENERATED_BODY()

	TArray<FPerfStatsRecord> Records;
	bool bRecordingBasicStats;
	bool bRecordingBaselineBasicStats;
	bool bRecordingCPUCapture;
	bool bRecordingStatsFile;

	/** If true we check the GPU times vs GPU budget each tick and trigger a GPU trace if we fall below budget.*/
	bool bGPUTraceIfBelowBudget;

public:

	UE_API UAutomationPerformaceHelper();

	// UObject interface
	UE_API virtual UWorld* GetWorld() const override;
	// End of UObject interface

	//Begin basic stat recording

	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void Tick(float DeltaSeconds);

	/** Adds a sample to the stats counters for the current performance stats record. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void Sample(float DeltaSeconds);
	/** Begins recording a new named performance stats record. We start by recording the baseline */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void BeginRecordingBaseline(FString RecordName);
	/** Stops recording the baseline and moves to the main record. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void EndRecordingBaseline();
	/** Begins recording a new named performance stats record. We start by recording the baseline. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void BeginRecording(FString RecordName,	float InGPUBudget, float InRenderThreadBudget, float InGameThreadBudget);
	/** Stops recording performance stats. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void EndRecording();
	/** Writes the current set of performance stats records to a csv file in the profiling directory. An additional directory and an extension override can also be used. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void WriteLogFile(const FString& CaptureDir, const FString& CaptureExtension);
	/** Returns true if this stats tracker is currently recording performance stats. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API bool IsRecording() const;

	/** Does any init work across all tests.. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void OnBeginTests();
	/** Does any final work needed as all tests are complete. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void OnAllTestsComplete();

	UE_API const FPerfStatsRecord* GetCurrentRecord() const;
	UE_API FPerfStatsRecord* GetCurrentRecord();

	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API bool IsCurrentRecordWithinGPUBudget() const;
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API bool IsCurrentRecordWithinGameThreadBudget() const;
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API bool IsCurrentRecordWithinRenderThreadBudget() const;
	//End basic stats recording.

	// Automatic traces capturing 

	/** Communicates with external profiler to being a CPU capture. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void StartCPUProfiling();
	/** Communicates with external profiler to end a CPU capture. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void StopCPUProfiling();
	/** Will trigger a GPU trace next time the current test falls below GPU budget. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void TriggerGPUTraceIfRecordFallsBelowBudget();
	/** Begins recording stats to a file. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void BeginStatsFile(const FString& RecordName);
	/** Ends recording stats to a file. */
	UFUNCTION(BlueprintCallable, Category = Perf)
	UE_API void EndStatsFile();

#if UE_EXTERNAL_PROFILING_ENABLED
	FFunctionalTestExternalProfiler ExternalProfiler;
#endif

	/** The path and base name for all output files. */
	FString OutputFileBase;
	
	FString StartOfTestingTime;
};

UENUM(BlueprintType)
enum class EFunctionalTestResult : uint8
{
	/**
	 * When finishing a test if you use Default, you're not explicitly stating if the test passed or failed.
	 * Instead you're instead allowing any tested assertions to have decided that for you.  Even if you do
	 * explicitly log success, it can be overturned by errors that occurred during the test.
	 */
	Default,
	Invalid,
	Error,
	Running,
	Failed,
	Succeeded
};

/* Return a readable string of the provided EFunctionalTestResult enum */
FString FUNCTIONALTESTING_API LexToString(const EFunctionalTestResult TestResult);

/** Return a dot-separated path prefix string representing the map that contains a test */
FString FUNCTIONALTESTING_API MapPackageToAutomationPath(const FString& MapPackageName);

/**
 * Return a dot-separated path prefix string representing the map that contains a test.
 * If the map defines a "TestPathOverride" AR tag, it'll take its value instead of building the default prefix.
 */
FString FUNCTIONALTESTING_API MapPackageToAutomationPath(const FAssetData& MapAsset);

/** Registration information for an individual test */
struct FFunctionalTestInfo
{
	FFunctionalTestInfo(FString BeautifiedName, FString TestCommand, FString TestTags)
		: BeautifiedName(BeautifiedName), TestCommand(TestCommand), TestTags(TestTags) {}

	FString BeautifiedName;
	FString TestCommand;
	FString TestTags;
};

UENUM(BlueprintType)
enum class EFunctionalTestLogHandling : uint8
{
	/**
	 * How do log categories affect rest results. ProjectDefault can be set in DefaultEngine.ini
	 * but individual tests can override that
	 */
	ProjectDefault,
	OutputIsError,
	OutputIgnored
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FFunctionalTestEventSignature);
DECLARE_DELEGATE_OneParam(FFunctionalTestDoneSignature, class AFunctionalTest*);

UCLASS(MinimalAPI, hidecategories=( Actor, Input, Rendering, HLOD ), Blueprintable)
class AFunctionalTest : public AActor
{
	GENERATED_BODY()

public:
	UE_API AFunctionalTest(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UPROPERTY(BlueprintReadOnly, Category = "Functional Testing")
	FString TestLabel;

	/**
	 * The owner is the group or person responsible for the test. Generally you should use a group name
	 * like 'Editor' or 'Rendering'. When a test fails it may not be obvious who should investigate
	 * so this provides a associate responsible groups with tests.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Functional Testing", meta = (MultiLine = "true", DisplayName = "Owner"))
	FString Author;

	/**
	 * A description of the test, like what is this test trying to determine.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Functional Testing", meta = (MultiLine = "true"))
	FString Description;

	/**
	 * Tags describing this test separated by square brackets, such as '[dog]' or '[cat]' or '[Graphics][prio0][unstable]'.
	 * Tags can be used to run subsets of tests, or to categorize data in test reports.
	 * Deprecating, please use "Edit Test Tags" instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Functional Testing")
	FString TestTags;

#if WITH_EDITOR
	/**
	 * Edit tags button that will open the Automation tab in the Session Frontend window then show the tag editor for this test.  
	 */
	UFUNCTION(CallInEditor, Category="Functional Testing", DisplayName = "Edit Test Tags", meta=(DisplayAfter="TestTags"))
	UE_API void EditTags();
#endif

	/** Return functional test beautified name */
	FString GetBeautifiedName();

private:
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;

	FString PreviousTestTags;

protected:
	/**
	 * Allows a test to be disabled.  If a test is disabled, it will not appear in the set of
	 * runnable tests (after saving the map).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Functional Testing", meta=(ScriptName="IsEnabledValue"))
	uint32 bIsEnabled:1;

	/**
	 * Allows a test to be enabled only if loaded from the specified persistent level; avoiding other levels to run the test.
	 */
	UPROPERTY(EditInstanceOnly, Category="Functional Testing")
	uint32 bIsInSublevel:1;

	/**
	 * Name of the persistent level to run the test from.
	 */
	UPROPERTY(EditInstanceOnly, Category="Functional Testing", meta=(EditCondition="bIsInSublevel"))
	FName PersistentLevelName;

	/**
	 * Determines how LogErrors are handled during this test.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Functional Testing", meta = (ScriptName = "LogErrorHandling"))
	EFunctionalTestLogHandling LogErrorHandling;

	/**
	 * Determines how LogWarnings are handled during this test.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Functional Testing", meta = (ScriptName = "LogWarningHandling"))
	EFunctionalTestLogHandling LogWarningHandling;

	/**
	 * Allows you to specify another actor to view the test from.  Usually this is a camera you place
	 * in the map to observe the test.  Not useful when running on a build farm, but provides a handy
	 * way to observe the test from a different location than you place the functional test actor.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Functional Testing")
	TObjectPtr<AActor> ObservationPoint;

	/**
	 * Allows for garbage collection to be delayed. If delayed, garbage collection will be triggered at the end of a test run
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Functional Testing", AdvancedDisplay)
	uint32 bShouldDelayGarbageCollection:1;

	/**
	 * A random number stream that you can use during testing.  This number stream will be consistent
	 * every time the test is run.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Functional Testing", AdvancedDisplay)
	FRandomStream RandomNumbersStream;

public:

	UPROPERTY(BlueprintReadWrite, Category="Functional Testing")
	EFunctionalTestResult Result;

	/** The Test's time limit for preparation, this is the time it has to return true when checking IsReady(). '0' means no limit. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Timeout")
	float PreparationTimeLimit;

	/** Test's time limit. '0' means no limit */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Timeout")
	float TimeLimit;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Timeout", meta=( MultiLine="true" ))
	FText TimesUpMessage;

	/** If test is limited by time this is the result that will be returned when time runs out */
	UPROPERTY(EditAnywhere, Category="Timeout")
	EFunctionalTestResult TimesUpResult;

public:

	//UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Rendering")
	//FQualityLevels

public:

	/** Called when the test is ready to prepare */
	UPROPERTY(BlueprintAssignable)
	FFunctionalTestEventSignature OnTestPrepare;

	/** Called when the test is started */
	UPROPERTY(BlueprintAssignable)
	FFunctionalTestEventSignature OnTestStart;

	/** Called when the test is finished. Use it to clean up */
	UPROPERTY(BlueprintAssignable)
	FFunctionalTestEventSignature OnTestFinished;

	UPROPERTY(Transient)
	TArray<TObjectPtr<AActor>> AutoDestroyActors;
	
	FString FailureMessage;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<class UFuncTestRenderingComponent> RenderComp;

	UPROPERTY()
	TObjectPtr<class UTextRenderComponent> TestName;
#endif // WITH_EDITORONLY_DATA

	/** List of causes we need a re-run. */
	TArray<FName> RerunCauses;

	/** Cause of the current rerun if we're in a named rerun. */
	FName CurrentRerunCause;

public:
	/**
	 * Assert that a boolean value is true.
	 * @param Message	The message to display if the assert fails ("Assertion Failed: 'Message' for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertTrue(bool Condition, const FString& Message, const UObject* ContextObject = nullptr);

	/**
	 * Assert that a boolean value is false.
	 * @param Message	The message to display if the assert fails ("Assertion Failed: 'Message' for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertFalse(bool Condition, const FString& Message, const UObject* ContextObject = nullptr);

	/**
	 * Assert that a UObject is valid
	 * @param Message	The message to display if the object is invalid ("Invalid object: 'Message' for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertIsValid(UObject* Object, const FString& Message, const UObject* ContextObject = nullptr);

	/**
	 * Assert on a relationship between two integers.
	 * @param What	A name to use in the message if the assert fails (What: expected {Actual} to be <ShouldBe> {Expected} for context '')
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Value (Integer)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertValue_Int(int32 Actual, EComparisonMethod ShouldBe, int32 Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert on a relationship between two floats.
	 * @param What	A name to use in the message if the assert fails (What: expected {Actual} to be <ShouldBe> {Expected} for context '')
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Value (Float)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertValue_Float(float Actual, EComparisonMethod ShouldBe, float Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert on a relationship between two doubles.
	 * @param What	A name to use in the message if the assert fails (What: expected {Actual} to be <ShouldBe> {Expected} for context '')
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Value (Double)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API bool AssertValue_Double(double Actual, EComparisonMethod ShouldBe, double Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert on a relationship between two DateTimes.
	 * @param What	A name to use in the message if the assert fails (What: expected {Actual} to be <ShouldBe> {Expected} for context '')
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Value (DateTime)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertValue_DateTime(FDateTime Actual, EComparisonMethod ShouldBe, FDateTime Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two transforms are (components memberwise - translation, rotation, scale) equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Transform)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Transform(const FTransform& Actual, const FTransform& Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two floats are equal within tolerance between two floats.
	 * @param What	A name to use in the message if the assert fails (What: expected {Actual} to be Equal To {Expected} within Tolerance for context '')
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Float)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Float(float Actual, float Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two double are equal within tolerance between two doubles.
	 * @param What	A name to use in the message if the assert fails (What: expected {Actual} to be Equal To {Expected} within Tolerance for context '')
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Double)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API bool AssertEqual_Double(double Actual, double Expected, const FString& What, double Tolerance = 1.e-4, const UObject* ContextObject = nullptr);

	/**
	* Assert that two bools are equal
	* @param What	A name to use in the message if the assert fails (What: expected {Actual} to be Equal To {Expected} for context '')
	*/
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Bool)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Bool(bool Actual, bool Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	* Assert that two ints are equal
	* @param What	A name to use in the message if the assert fails (What: expected {Actual} to be Equal To {Expected} for context '')
	*/
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Integer)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Int(int Actual, int Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	* Assert that two FNames are equal
	* @param What	A name to use in the message if the assert fails (What: expected {Actual} to be Equal To {Expected} for context '')
	*/
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (FName)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Name(FName Actual, FName Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	* Assert that two Objects are equal
	* @param What	A name to use in the message if the assert fails (What: expected {Actual} to be Equal To {Expected} for context '')
	*/
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Object)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Object(UObject* Actual, UObject* Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two transforms are (components memberwise - translation, rotation, scale) not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Transform)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Transform(const FTransform& Actual, const FTransform& NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that the component angles of two rotators are all equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Rotator)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Rotator(FRotator Actual, FRotator Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that the orientation of two rotators is the same within a small tolerance. Robust to quaternion singularities where angles can differ despite having an identical orientation. 
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Rotator Orientation)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_RotatorOrientation(FRotator Actual, FRotator Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that the component angles of two rotators are all not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Rotator)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Rotator(FRotator Actual, FRotator NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two vectors are (memberwise) equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Vector)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Vector(FVector Actual, FVector Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two vectors are (memberwise) not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Vector)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Vector(FVector Actual, FVector NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two two-component vectors are (memberwise) equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Vector2D)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Vector2D(FVector2D Actual, FVector2D Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two two-component vectors are (memberwise) not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Vector2D)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Vector2D(FVector2D Actual, FVector2D NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two two-component boxes are (memberwise) equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Box2D)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Box2D(FBox2D Actual, FBox2D Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two two-component boxes are (memberwise) not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Box2D)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Box2D(FBox2D Actual, FBox2D NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two four-component vectors are (memberwise) equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Vector4)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Vector4(FVector4 Actual, FVector4 Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two four-component vectors are (memberwise) not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Vector4)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Vector4(FVector4 Actual, FVector4 NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two planes are (memberwise) equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Plane)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Plane(FPlane Actual, FPlane Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two planes are (memberwise) not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Plane)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Plane(FPlane Actual, FPlane NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two quats are (memberwise) equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Quat)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Quat(FQuat Actual, FQuat Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two quats are (memberwise) not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Quat)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Quat(FQuat Actual, FQuat NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two 4x4 matrices are (memberwise) equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (Matrix)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_Matrix(FMatrix Actual, FMatrix Expected, const FString& What, float Tolerance = 1.e-4f, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two 4x4 matrices are (memberwise) not equal within a small tolerance.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (Matrix)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_Matrix(FMatrix Actual, FMatrix NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two Strings are equal.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (String)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_String(FString Actual, FString Expected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	 * Assert that two Strings are not equal.
	 * @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	 */
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Not Equal (String)", meta = ( HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertNotEqual_String(FString Actual, FString NotExpected, const FString& What, const UObject* ContextObject = nullptr);

	/**
	* Assert that two TraceQueryResults are equal.
	* @param What	A name to use in the message if the assert fails ("Expected 'What' not to be {Expected} but it was {Actual} for context ''")
	*/
	UFUNCTION(BlueprintCallable, Category = "Asserts", DisplayName = "Assert Equal (TraceQuery)", meta = (HidePin = "ContextObject", DefaultToSelf = "ContextObject"))
	UE_API virtual bool AssertEqual_TraceQueryResults(const UTraceQueryTestResults* Actual, const UTraceQueryTestResults* Expected, const FString& What, const UObject* ContextObject = nullptr);

	UFUNCTION(BlueprintCallable, Category = "Reporting")
	UE_API void AddWarning(const FString& Message);

	UFUNCTION(BlueprintCallable, Category = "Reporting")
	UE_API virtual void AddError(const FString& Message);

	UFUNCTION(BlueprintCallable, Category = "Reporting")
	UE_API virtual void AddInfo(const FString& Message);

//protected:
	/** TODO: break this out into a library */
	UE_API void LogStep(ELogVerbosity::Type Verbosity, const FString& Message);

public:
	UE_API virtual bool RunTest(const TArray<FString>& Params = TArray<FString>());

public:
	UE_API FString	GetCurrentStepName() const;
	UE_API void 	StartStep(const FString& StepName);
	UE_API void 	FinishStep();
	UE_API bool	IsInStep() const;

	UFUNCTION(BlueprintCallable, Category="Functional Testing")
	UE_API virtual void FinishTest(EFunctionalTestResult TestResult, const FString& Message);

	UFUNCTION(BlueprintCallable, Category="Functional Testing")
	UE_API virtual void LogMessage(const FString& Message);

	UFUNCTION(BlueprintCallable, Category="Functional Testing")
	UE_API virtual void SetTimeLimit(float NewTimeLimit, EFunctionalTestResult ResultWhenTimeRunsOut);

public:

	/** Used by debug drawing to gather actors this test is using and point at them on the level to better understand test's setup */
	UFUNCTION(BlueprintImplementableEvent, CallInEditor, Category="Functional Testing")
	UE_API TArray<AActor*> DebugGatherRelevantActors() const;

	UE_API virtual void GatherRelevantActors(TArray<AActor*>& OutActors) const;

	/** retrieves information whether test wants to have another run just after finishing */
	UFUNCTION(BlueprintImplementableEvent, Category="Functional Testing")
	UE_API bool OnWantsReRunCheck() const;

	virtual bool WantsToRunAgain() const { return false; }

	/** Causes the test to be rerun for a specific named reason. */
	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API void AddRerun(FName Reason);

	/** Returns the current re-run reason if we're in a named re-run. */
	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API FName GetCurrentRerunReason() const;

	/** Sets the CVar from the given input. Variable gets reset after the test. */
	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API void SetConsoleVariable(const FString& Name, const FString& InValue);

	/** Sets the CVar from the given input. Variable gets reset after the test. */
	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API void SetConsoleVariableFromInteger(const FString& Name, const int32 InValue);

	/** Sets the CVar from the given input. Variable gets reset after the test. */
	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API void SetConsoleVariableFromFloat(const FString& Name, const float InValue);

	/** Sets the CVar from the given input. Variable gets reset after the test. */
	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API void SetConsoleVariableFromBoolean(const FString& Name, const bool InValue);

	UFUNCTION(BlueprintImplementableEvent, Category = "Functional Testing")
	UE_API FString OnAdditionalTestFinishedMessageRequest(EFunctionalTestResult TestResult) const;
	
	virtual FString GetAdditionalTestFinishedMessage(EFunctionalTestResult TestResult) const { return FString(); }

public:
	
	/** Actors registered this way will be automatically destroyed (by limiting their lifespan)
	 *	on test finish */
	UFUNCTION(BlueprintCallable, Category="Development", meta=(Keywords = "Delete"))
	UE_API virtual void RegisterAutoDestroyActor(AActor* ActorToAutoDestroy);

	/** Called to clean up when tests is removed from the list of active tests after finishing execution. 
	 *	Note that FinishTest gets called after every "cycle" of a test (where further cycles are enabled by  
	 *	WantsToRunAgain calls). CleanUp gets called when all cycles are done. */
	UE_API virtual void CleanUp();

	virtual FString GetReproString() const { return GetFName().ToString(); }

#if WITH_EDITOR
	UE_API virtual void PostLoad() override;
	UE_API void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	static UE_API void OnSelectObject(UObject* NewSelection);
#endif // WITH_EDITOR

	// AActor interface begin
	UE_API virtual void OnConstruction(const FTransform& Transform) override;
	UE_API virtual void Tick(float DeltaSeconds) override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
	virtual bool ActorTypeSupportsDataLayer() const override { return false; }
	virtual bool ActorTypeSupportsExternalDataLayer() const override { return false; }
#endif
	// AActor interface end

	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API bool IsRunning() const;

	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API bool IsEnabled() const;

	UFUNCTION(BlueprintCallable, Category = "Functional Testing")
	UE_API bool IsEnabledInWorld(const UWorld* World) const;

protected:
	/**
	 * Prepare Test is fired once the test starts up, before the test IsReady() and thus before Start Test is called.
	 * So if there's some initial conditions or setup that you might need for your IsReady() check, you might want
	 * to do that here.
	 */
	UE_API virtual void PrepareTest();

	/**
	 * Prepare Test is fired once the test starts up, before the test IsReady() and thus before Start Test is called.
	 * So if there's some initial conditions or setup that you might need for your IsReady() check, you might want
	 * to do that here.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=( DisplayName="Prepare Test" ))
	UE_API void ReceivePrepareTest();

	/**
	 * Called once the IsReady() check for the test returns true.  After that happens the test has Officially started,
	 * and it will begin receiving Ticks in the blueprint.
	 */
	UE_API virtual void StartTest();
	
	/**
	 * Called once the IsReady() check for the test returns true.  After that happens the test has Officially started,
	 * and it will begin receiving Ticks in the blueprint.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=( DisplayName="Start Test" ))
	UE_API void ReceiveStartTest();

	/**
	 * Called during FinishTest().  Allows for clean-up on the blueprint side, so that the user can bring the test
	 * back to its default state, making it ready for the next one.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=( DisplayName="Test Finished" ))
	UE_API void ReceiveTestFinished();

	/**
	 * IsReady() is called once per frame after a test is run, until it returns true.  You should use this function to
	 * delay Start being called on the test until preconditions are met.
	 */
	UFUNCTION(BlueprintNativeEvent, Category="Functional Testing")
	UE_API bool IsReady();

	UE_API virtual bool IsReady_Implementation();

	UE_API virtual void OnTimeout();

	/**
	 * Goto an observation location.
	 */
	UE_API void GoToObservationPoint();

public:
	FFunctionalTestDoneSignature TestFinishedObserver;

	// AG TEMP - solving a compile issue in a temp way to unblock the build
	UPROPERTY(Transient)
	bool bIsRunning;

	TArray<FString> Steps;
	
	UPROPERTY(BlueprintReadOnly, Category = "Functional Testing")
	float TotalTime;

	uint32 RunFrame;
	float RunTime;

	uint32 StartFrame;
	float StartTime;

private:
	bool bIsReady;
	TSharedPtr<FScopedTestEnvironment> EnvSetup;

public:
	/** Returns SpriteComponent subobject **/
	UE_API UBillboardComponent* GetSpriteComponent();
};

#undef UE_API
