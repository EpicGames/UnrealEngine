// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

/**
 * Base class for Functional test cases.
 */
class FFunctionalTestBase : public FAutomationTestBase
{
public:
	FUNCTIONALTESTING_API FFunctionalTestBase(const FString& InName, const bool bInComplexTask);

	/**
	 * If true logs will not be included in test events
	 *
	 * @return true to suppress logs
	 */
	FUNCTIONALTESTING_API virtual bool SuppressLogs();

	/**
	 * Should the log category be captured and surfaced as part of the test.
	 *
	 * @return true to allow a log category through.
	 */
	FUNCTIONALTESTING_API virtual bool ShouldCaptureLogCategory(const class FName& Category) const;


	/**
	 * Specify how log errors & warnings should be handled during tests. If values are not set then the project
	 * defaults will be used.
	 */
	void SetLogErrorAndWarningHandling(TOptional<bool> InSuppressErrors, TOptional<bool> InSuppressWarnings, TOptional<bool> InWarningsAreErrors)
	{
		SetLogErrorAndWarningHandlingToDefault();

		if (InSuppressErrors.IsSet())
		{
			bSuppressLogErrors = InSuppressErrors.GetValue();
		}

		if (InSuppressWarnings.IsSet())
		{
			bSuppressLogWarnings = InSuppressWarnings.GetValue();
		}

		if (InWarningsAreErrors.IsSet())
		{
			bElevateLogWarningsToErrors = InWarningsAreErrors.GetValue();
		}
	}

	/**
	 * Determines if Error logs should be suppressed from test results
	 */
	virtual bool SuppressLogErrors() override
	{
		return bSuppressLogErrors;
	}

	/**
	 * Determines if Warning logs should be suppressed from test results
	 */
	virtual bool SuppressLogWarnings() override
	{
		return bSuppressLogWarnings;
	}

	/**
	 * Determines if Warning logs should be treated as errors
	 */
	virtual bool ElevateLogWarningsToErrors() override
	{
		return bElevateLogWarningsToErrors;
	}

	/**
	 * Marks us as actively running a functional test
	 */
	FUNCTIONALTESTING_API void SetFunctionalTestRunning(const FString& InName);

	/**
	 * Marks us as no longer running a test
	 */
	FUNCTIONALTESTING_API void SetFunctionalTestComplete(const FString& InName);

	/**
	 * Returns the name of the running functional test. Empty if no test is running
	 */
	static FString GetRunningTestName() { return ActiveTestName; }

	/**
	 * Returns true if a functional test is running (does not include map setup)
	 */
	static bool IsFunctionalTestRunning() { return bIsFunctionalTestRunning; }

protected:

	FUNCTIONALTESTING_API void SetLogErrorAndWarningHandlingToDefault();

	bool bSuppressLogErrors;
	bool bSuppressLogWarnings;
	bool bElevateLogWarningsToErrors;
	bool bSuppressLogs;

private:

	FUNCTIONALTESTING_API static bool bIsFunctionalTestRunning;
	static FString ActiveTestName;
};
