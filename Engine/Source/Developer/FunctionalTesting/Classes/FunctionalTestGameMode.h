// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/GameModeBase.h"
#include "FunctionalTestGameMode.generated.h"

UCLASS(MinimalAPI, Blueprintable)
class AFunctionalTestGameMode : public AGameModeBase
{
	GENERATED_UCLASS_BODY()

};
