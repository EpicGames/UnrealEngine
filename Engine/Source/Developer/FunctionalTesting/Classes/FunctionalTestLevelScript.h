// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/LevelScriptActor.h"
#include "FunctionalTestLevelScript.generated.h"

UCLASS(MinimalAPI, notplaceable, meta=(KismetHideOverrides = "ReceiveAnyDamage,ReceivePointDamage,ReceiveRadialDamage,ReceiveActorBeginOverlap,ReceiveActorEndOverlap,ReceiveHit,ReceiveDestroyed,ReceiveActorBeginCursorOver,ReceiveActorEndCursorOver,ReceiveActorOnClicked,ReceiveActorOnReleased,ReceiveActorOnInputTouchBegin,ReceiveActorOnInputTouchEnd,ReceiveActorOnInputTouchEnter,ReceiveActorOnInputTouchLeave"), HideCategories=(Collision,Rendering,Transformation))
class AFunctionalTestLevelScript : public ALevelScriptActor
{
	GENERATED_UCLASS_BODY()
	
};
