// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/KismetSystemLibrary.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "TraceQueryTestResults.h"
#include "FunctionalTestUtilityLibrary.generated.h"

#define UE_API FUNCTIONALTESTING_API

/**
* Used to expose C++ functions to tests that we don't want to make BP accessible
* in the engine itself.
*/
UCLASS(MinimalAPI)
class UFunctionalTestUtilityLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Helper function to trace and permute many options at once */
	UFUNCTION(BlueprintCallable, Category = "Utilities|Collision", meta = (bIgnoreSelf = "true", WorldContext = "WorldContextObject", AutoCreateRefTerm = "ActorsToIgnore", AdvancedDisplay = "TraceColor,TraceHitColor,DrawTime", Keywords = "sweep"))
	static UE_API UTraceQueryTestResults* TraceChannelTestUtil(UObject* WorldContextObject, const FTraceChannelTestBatchOptions& BatchOptions, const FVector Start, const FVector End, float SphereCapsuleRadius, float CapsuleHalfHeight, FVector BoxHalfSize, const FRotator Orientation, ETraceTypeQuery TraceChannel, TArray<TEnumAsByte<EObjectTypeQuery> > ObjectTypes, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, bool bIgnoreSelf, EDrawDebugTrace::Type DrawDebugType, FLinearColor TraceColor = FLinearColor::Red, FLinearColor TraceHitColor = FLinearColor::Green, float DrawTime = 5.0f);
};

#undef UE_API
