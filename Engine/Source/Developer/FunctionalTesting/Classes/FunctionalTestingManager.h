// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "FunctionalTest.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "FunctionalTestingManager.generated.h"

class IMessageLogListing;

namespace FFunctionalTesting
{
	extern const TCHAR* ReproStringTestSeparator;
	extern const TCHAR* ReproStringParamsSeparator;
}

#if WITH_EDITOR
namespace UE::Private 
{
	class FTickEditorTest;
}
#endif

UCLASS(BlueprintType, MinimalAPI)
class UFunctionalTestingManager : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<class AFunctionalTest>> TestsLeft;

	UPROPERTY(Transient)
	TArray<TObjectPtr<class AFunctionalTest>> AllTests;

	UPROPERTY(BlueprintAssignable)
	FFunctionalTestEventSignature OnSetupTests;

	UPROPERTY(BlueprintAssignable)
	FFunctionalTestEventSignature OnTestsComplete;

	UPROPERTY(BlueprintAssignable)
	FFunctionalTestEventSignature OnTestsBegin;
	
	/**
	 * Triggers in sequence all functional tests found on the level.
	 * @return true if any tests have been triggered
	 */
	UFUNCTION(BlueprintCallable, Category="FunctionalTesting", meta=(WorldContext="WorldContextObject", CallableWithoutWorldContext ) )
	static bool RunAllFunctionalTests(UObject* WorldContextObject, bool bNewLog = true, bool bRunLooped = false, FString FailedTestsReproString = TEXT(""));

	bool IsRunning() const { return bIsRunning; }
	bool IsFinished() const { return bFinished; }
	bool IsLooped() const { return bLooped; }
	void SetLooped(const bool bNewLooped) { bLooped = bNewLooped; }
	
protected:
	static UFunctionalTestingManager* GetManager(UObject* WorldContext);

	void TriggerFirstValidTest();
	void SetUpTests();

	void OnTestDone(class AFunctionalTest* FTest);

	bool RunFirstValidTest();

	void NotifyTestDone(class AFunctionalTest* FTest);

	void SetReproString(FString ReproString);
	void AllTestsDone();

	void OnWorldCleanedUp(UWorld* World, bool bSessionEnded, bool bCleanupResources);
	virtual UWorld* GetWorld() const override;
	
	bool bIsRunning;
	bool bFinished;
	bool bLooped;
	bool bInitialDelayApplied;
    bool bIsTearingDown;
	uint32 CurrentIteration;

	FFunctionalTestDoneSignature TestFinishedObserver;
	
	//FString GatheredFailedTestsReproString;
	FString StartingReproString;
	TArray<FString> TestReproStrings;

private:

#if WITH_EDITOR
	TUniquePtr<UE::Private::FTickEditorTest> EditorTick;
#endif

	FTimerHandle TriggerFirstValidTestTimerHandle;
};

UCLASS(abstract, Blueprintable, MinimalAPI)
class APhasedAutomationActorBase : public AActor
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintImplementableEvent, Category = "Automation")
	void OnFunctionalTestingComplete();

	UFUNCTION(BlueprintImplementableEvent, Category = "Automation")
	void OnFunctionalTestingBegin();
};
