// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ScreenshotFunctionalTestBase.h"
#include "AutomationScreenshotOptions.h"
#include "Blueprint/UserWidget.h"
#include "Delegates/IDelegateInstance.h"

#include "FunctionalUIScreenshotTest.generated.h"

class UTextureRenderTarget2D;

UENUM()
enum class EWidgetTestAppearLocation
{
	Viewport,
	PlayerScreen
};

/**
 * 
 */
UCLASS(Blueprintable, MinimalAPI)
class AFunctionalUIScreenshotTest : public AScreenshotFunctionalTestBase
{
	GENERATED_BODY()

public:
	AFunctionalUIScreenshotTest(const FObjectInitializer& ObjectInitializer);

public:
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

protected:
	virtual void PrepareTest() override;

	virtual void OnScreenshotTakenAndCompared() override;

	virtual void RequestScreenshot() override;

	virtual bool IsReady_Implementation() override;

	virtual void Tick(float DeltaSeconds) override;

protected:
	UPROPERTY(EditAnywhere, Category = "UI")
	TSubclassOf<UUserWidget> WidgetClass;

	UPROPERTY()
	TObjectPtr<UUserWidget> SpawnedWidget;

	UPROPERTY(EditAnywhere, Category = "UI")
	EWidgetTestAppearLocation WidgetLocation;

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UTextureRenderTarget2D> ScreenshotRT;

	UPROPERTY(EditAnywhere, Category = "UI")
	bool bHideDebugCanvas;

private:
	TOptional<bool> PreviousDebugCanvasVisible;
	int32 NumTickPassed;
	bool bWasPreviouslyUsingFixedDeltaTime;
	double PreviousFixedDeltaTime;
	const double TestFixedDeltaTime = 1 / 60.0;

	struct FScreenshotData
	{
		TArray<FColor> ColorData;
		FIntPoint Size;
	};

	TFuture<FScreenshotData> Future;
	FDelegateHandle Handle;

	static void ReadBackBufferData(TPromise<FScreenshotData>& Promise, FRHITexture* Texture);
};
