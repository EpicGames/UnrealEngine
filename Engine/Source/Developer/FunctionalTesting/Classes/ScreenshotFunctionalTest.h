// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ScreenshotFunctionalTestBase.h"
#include "AutomationScreenshotOptions.h"

#include "ScreenshotFunctionalTest.generated.h"

#define UE_API FUNCTIONALTESTING_API

class FAutomationTestScreenshotEnvSetup;

/**
 * No UI
 */
UCLASS(MinimalAPI, Blueprintable)
class AScreenshotFunctionalTest : public AScreenshotFunctionalTestBase
{
	GENERATED_BODY()

public:
	UE_API AScreenshotFunctionalTest(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void Serialize(FArchive& Ar) override;

	// Tests not relying on temporal effects can force a camera cut to flush stale data
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera", SimpleDisplay)
	bool bCameraCutOnScreenshotPrep;


protected:
	UE_API virtual bool RunTest(const TArray<FString>& Params = TArray<FString>()) override;
	UE_API virtual void PrepareTest() override;
	UE_API virtual bool IsReady_Implementation() override;
	UE_API virtual void Tick(float DeltaSeconds) override;

	UE_API virtual void RequestScreenshot() override;
	UE_API virtual void OnScreenShotCaptured(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData) override;
	UE_API virtual void OnScreenshotTakenAndCompared() override;
	

private:

	enum EVariantType
	{
		Baseline = 0,
		ViewRectOffset,
		Num
	};

	struct FVariantInfo
	{
		const TCHAR* Name;
		const TCHAR* SetupCommand;
		const TCHAR* RestoreCommand;
	};

	bool SetupNextVariant();
	void SetupVariant(EVariantType VariantType);
	
	TBitArray<> RequestedVariants;
	const TCHAR* CurrentVariantName;
	const TCHAR* VariantRestoreCommand;

	// Used for subsequent variants after the first is done
	void ReprepareTest();

	uint32 VariantFrame;
	float VariantTime;
	bool bVariantQueued;
};

#undef UE_API
