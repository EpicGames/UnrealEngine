// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "FunctionalTest.h"
#include "AutomationScreenshotOptions.h"
#include "Misc/AutomationTest.h"

#include "ScreenshotFunctionalTestBase.generated.h"

#define UE_API FUNCTIONALTESTING_API

DEFINE_LOG_CATEGORY_STATIC(LogScreenshotFunctionalTest, Log, Log)

class FAutomationTestScreenshotEnvSetup;

/**
* Base class for screenshot functional test
*/
UCLASS(MinimalAPI, Blueprintable, abstract)
class AScreenshotFunctionalTestBase : public AFunctionalTest
{
	GENERATED_BODY()

public:
	UE_API AScreenshotFunctionalTestBase(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual void FinishTest(EFunctionalTestResult TestResult, const FString& Message) override;

protected:
	// Set player view target to screenshot camera and call PrepareForScreenshot
	UE_API virtual void PrepareTest() override;

	// Handle screenshot delay
	UE_API virtual bool IsReady_Implementation() override;

	// Register OnScreenshotTakenAndCompared and call RequestScreenshot
	UE_API virtual void StartTest() override;

	// Call RestoreViewport and finish this test
	UE_API virtual void OnScreenshotTakenAndCompared();

	// Resize viewport to screenshot size (if possible) and set up screenshot environment (disable AA, etc.)
	UE_API void PrepareForScreenshot();

	// Doesn't actually request in base class. It simply register OnScreenshotCaptured
	UE_API virtual void RequestScreenshot();

	// Pass screenshot pixels and meta data to FAutomationTestFramework. Register
	// OnComparisonComplete which will be called the automation test system when
	// screenshot comparison is complete
	UE_API virtual void OnScreenShotCaptured(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData);

	// Do some logging and trigger OnScreenshotTakenAndCompared
	UE_API void OnComparisonComplete(const FAutomationScreenshotCompareResults& CompareResults);

	// Restore viewport size and original environment settings
	UE_API void RestoreViewSettings();

	UE_API virtual void OnTimeout() override;

protected:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Screenshot", meta = (MultiLine = "true"))
	FString Notes;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Screenshot")
	TObjectPtr<class UCameraComponent> ScreenshotCamera;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Screenshot", SimpleDisplay)
	FAutomationScreenshotOptions ScreenshotOptions;

	FIntPoint ViewportRestoreSize;

#if WITH_AUTOMATION_TESTS
	TSharedPtr<FAutomationTestScreenshotEnvSetup> ScreenshotEnvSetup;
#endif

private:
	bool bNeedsViewSettingsRestore;
	bool bNeedsViewportRestore;
	bool bScreenshotCompleted;
};

#undef UE_API
