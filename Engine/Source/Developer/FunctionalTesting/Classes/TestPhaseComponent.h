// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Components/SceneComponent.h"
#include "TestPhaseComponent.generated.h"

UCLASS(hidecategories=Object, editinlinenew)
class UTestPhaseComponent : public USceneComponent
{
	GENERATED_BODY()

public:
	UTestPhaseComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//virtual void Start();
};
