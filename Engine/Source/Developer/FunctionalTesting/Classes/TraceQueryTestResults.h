// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/HitResult.h"
#include "TraceQueryTestResults.generated.h"

#define UE_API FUNCTIONALTESTING_API

class AFunctionalTest;

USTRUCT(BlueprintType)
struct FTraceChannelTestBatchOptions
{
	GENERATED_BODY()

	FTraceChannelTestBatchOptions()
	{
		bLineTrace = true;
		bSphereTrace = false;
		bCapsuleTrace = false;
		bBoxTrace = false;

		bChannelTrace = true;
		bObjectsTrace = false;
		bProfileTrace = false;
	}

	bool operator==(const FTraceChannelTestBatchOptions& Other) const
	{
		return bLineTrace == Other.bLineTrace && bSphereTrace == Other.bSphereTrace && bCapsuleTrace == Other.bCapsuleTrace && bBoxTrace == Other.bBoxTrace
			&& bChannelTrace == Other.bChannelTrace && bObjectsTrace == Other.bObjectsTrace && bProfileTrace == Other.bProfileTrace;
	}

	bool operator!=(const FTraceChannelTestBatchOptions& Other) const
	{
		return !(*this == Other);
	}

	/** Whether to do line traces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bLineTrace;

	/** Whether to do sphere traces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bSphereTrace;

	/** Whether to do capsule traces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bCapsuleTrace;

	/** Whether to do box traces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bBoxTrace;

	/** Whether to do channel traces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bChannelTrace;

	/** Whether to do object traces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bObjectsTrace;

	/** Whether to do profile traces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bProfileTrace;

	UE_API FString ToString() const;
};

USTRUCT(BlueprintType)
struct FTraceQueryTestNames
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FName ComponentName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FName PhysicalMaterialName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FName ActorName;

	UE_API FString ToString() const;
};

USTRUCT(BlueprintType)
struct FTraceQueryTestResultsInnerMost
{
	GENERATED_BODY()

	/** Result from doing a single sweep*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FHitResult SingleHit;

	/** Names found from doing a single sweep*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceQueryTestNames SingleNames;

	/** The true/false value returned from the single sweep */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bSingleResult = false;

	/** Result from doing a multi sweep */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	TArray<FHitResult> MultiHits;

	/** Names found from doing a multi sweep*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	TArray<FTraceQueryTestNames> MultiNames;

	/** The true/false value returned from the multi sweep */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	bool bMultiResult = false;

	UE_API FString ToString() const;

	UE_API void CaptureNames();

	UE_API bool AssertEqual(const FTraceQueryTestResultsInnerMost& Expected, const FString& What, const UObject* ContextObject, AFunctionalTest& FunctionalTest) const;
};

USTRUCT(BlueprintType)
struct FTraceQueryTestResultsInner
{
	GENERATED_BODY()

	/** The results associated with the line trace */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceQueryTestResultsInnerMost LineResults;

	/** The results associated with the sphere */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceQueryTestResultsInnerMost SphereResults;

	/** The results associated with the capsule*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceQueryTestResultsInnerMost CapsuleResults;

	/** The results associated with the box*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceQueryTestResultsInnerMost BoxResults;
	
	UE_API FString ToString(const FTraceChannelTestBatchOptions& BatchOptions) const;

	UE_API void CaptureNames();

	UE_API bool AssertEqual(const FTraceQueryTestResultsInner& Expected, const FString& What, const UObject* ContextObject, AFunctionalTest& FunctionalTest) const;
};

UCLASS(MinimalAPI, BlueprintType)
class UTraceQueryTestResults : public UObject
{
	GENERATED_BODY()
public:
	/** Results for channel trace */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceQueryTestResultsInner ChannelResults;

	/** Results for object trace */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceQueryTestResultsInner ObjectResults;

	/** Results for profile trace */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceQueryTestResultsInner ProfileResults;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Utility|Collision")
	FTraceChannelTestBatchOptions BatchOptions;

	/** Output string value */
	UFUNCTION(BlueprintCallable, Category = "Utility|Collision")
		UE_API FString ToString();

	UE_API bool AssertEqual(const UTraceQueryTestResults* Expected, const FString& What, const UObject* ContextObject, AFunctionalTest& FunctionalTest) const;

	UE_API void CaptureNames();

	
};

#undef UE_API
