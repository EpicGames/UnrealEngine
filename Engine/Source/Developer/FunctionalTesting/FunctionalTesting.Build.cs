// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class FunctionalTesting : ModuleRules
{
    public FunctionalTesting(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "Engine",
                "RenderCore",
                "Slate",
				"SlateCore",
                "MessageLog",
                "NavigationSystem",
                "AIModule",
                "RenderCore",
                "AssetRegistry",
                "RHI",
                "UMG",
				"AutomationController",
				"ImageWrapper",
            }
        );

        if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SourceControl",
					"EditorFramework",
					"UnrealEd",
					"LevelEditor",
					"SessionFrontend"
				}
			);

			// Circular references that need to be cleaned up
			CircularlyReferencedDependentModules.AddRange(
				new string[] {
					"UnrealEd",
				}
			);
		}

		//make sure this is compiled for binary builds
		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}
	}
}
