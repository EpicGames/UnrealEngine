// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationBlueprintFunctionLibrary.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AutomationTest.h"
#include "EngineGlobals.h"
#include "UnrealClient.h"
#include "Camera/CameraActor.h"
#include "Camera/PlayerCameraManager.h"
#include "Engine/Texture.h"
#include "Engine/GameViewportClient.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/Engine.h"
#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Editor.h"
#include "HighResScreenshot.h"
#include "LevelEditor.h"
#include "IAssetViewport.h"
#include "LevelEditorViewport.h"
#endif
#include "Tests/AutomationCommon.h"
#include "Logging/MessageLog.h"
#include "TakeScreenshotAfterTimeLatentAction.h"
#include "HighResScreenshot.h"
#include "Slate/SceneViewport.h"
#include "Tests/AutomationTestSettings.h"
#include "Slate/WidgetRenderer.h"
#include "DelayAction.h"
#include "Widgets/SViewport.h"
#include "Framework/Application/SlateApplication.h"
#include "ShaderCompiler.h"
#include "AutomationBlueprintFunctionLibrary.h"
#include "BufferVisualizationData.h"
#include "Engine/LocalPlayer.h"
#include "ContentStreaming.h"
#include "Stats/StatsData.h"
#include "HAL/PlatformProperties.h"
#include "IAutomationControllerModule.h"
#include "Scalability.h"
#include "SceneViewExtension.h"
#include "SceneView.h"
#include "Engine/GameEngine.h"
#include "Engine/LevelStreaming.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/GCObjectScopeGuard.h"
#include "Containers/Ticker.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageWrapperHelper.h"
#include "Misc/FileHelper.h"
#include "Materials/MaterialInterface.h"
#include "AssetCompilingManager.h"
#include "DynamicResolutionState.h"

#if WITH_EDITOR
#include "SLevelViewport.h"
#endif
#include "FunctionalTestBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationBlueprintFunctionLibrary)


#define LOCTEXT_NAMESPACE "Automation"

DEFINE_LOG_CATEGORY_STATIC(BlueprintAssertion, Error, Error)
DEFINE_LOG_CATEGORY_STATIC(AutomationFunctionLibrary, Log, Log)

static TAutoConsoleVariable<int32> CVarAutomationScreenshotResolutionWidth(
	TEXT("AutomationScreenshotResolutionWidth"),
	0,
	TEXT("The width of automation screenshots."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarAutomationScreenshotResolutionHeight(
	TEXT("AutomationScreenshotResolutionHeight"),
	0,
	TEXT("The height of automation screenshots."),
	ECVF_Default);


bool UAutomationEditorTask::IsValidTask() const
{
	return Task.IsValid();
}

void UAutomationEditorTask::BindTask(TUniquePtr<FAutomationTaskStatusBase> inTask)
{
	Task = MoveTemp(inTask);
}

bool UAutomationEditorTask::IsTaskDone() const
{
	return IsValidTask() && Task->IsDone();
}

#if WITH_AUTOMATION_TESTS

template<typename T>
FConsoleVariableSwapperTempl<T>::FConsoleVariableSwapperTempl(FString InConsoleVariableName)
	: bModified(false)
	, ConsoleVariableName(InConsoleVariableName)
{
}

template<typename T>
void FConsoleVariableSwapperTempl<T>::Set(T Value)
{
	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName);
	if (ensure(ConsoleVariable))
	{
		if (bModified == false)
		{
			bModified = true;
			OriginalValue = ConsoleVariable->GetInt();
		}

		ConsoleVariable->AsVariable()->SetWithCurrentPriority(Value);
	}
}

template<>
void FConsoleVariableSwapperTempl<float>::Set(float Value)
{
	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName);
	if (ensure(ConsoleVariable))
	{
		if (bModified == false)
		{
			bModified = true;
			OriginalValue = ConsoleVariable->GetFloat();
		}

		// I need these overrides to superseded anything the user does while taking the shot.
		ConsoleVariable->AsVariable()->SetWithCurrentPriority(Value);
	}
}

template<typename T>
void FConsoleVariableSwapperTempl<T>::Restore()
{
	if (bModified)
	{
		IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*ConsoleVariableName);
		if (ensure(ConsoleVariable))
		{
			// First we stomp the current with the original, then restore the original flags
			// so that code continues to treat it using whatever source it was from originally, code, cmdline..etc.
			ConsoleVariable->AsVariable()->SetWithCurrentPriority(OriginalValue);
		}

		bModified = false;
	}
}

class FAutomationViewExtension : public FWorldSceneViewExtension
{
public:
	FAutomationViewExtension(const FAutoRegister& AutoRegister, UWorld* InWorld, FAutomationScreenshotOptions& InOptions, float InCurrentTimeToSimulate)
		: FWorldSceneViewExtension(AutoRegister, InWorld)
		, Options(InOptions)
	{
	}

	/** ISceneViewExtension interface */
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
	{
		//if (Options.VisualizeBuffer != NAME_None)
		//{
		//	InViewFamily.ViewMode = VMI_VisualizeBuffer;
		//	InViewFamily.EngineShowFlags.SetVisualizeBuffer(true);
		//	InViewFamily.EngineShowFlags.SetTonemapper(false);

		//	if (GetBufferVisualizationData().GetMaterial(Options.VisualizeBuffer) == NULL)
		//	{
		//		InView.CurrentBufferVisualizationMode = Options.VisualizeBuffer;
		//	}
		//}
	}

	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override
	{
		if (UAutomationViewSettings* ViewSettings = Options.ViewSettings)
		{
			// Turn off common show flags for noisy sources of rendering.
			FEngineShowFlags& ShowFlags = InViewFamily.EngineShowFlags;
			ShowFlags.SetAntiAliasing(ViewSettings->AntiAliasing);
			ShowFlags.SetMotionBlur(ViewSettings->MotionBlur);
			ShowFlags.SetTemporalAA(ViewSettings->TemporalAA);
			ShowFlags.SetScreenSpaceReflections(ViewSettings->ScreenSpaceReflections);
			ShowFlags.SetScreenSpaceAO(ViewSettings->ScreenSpaceAO);
			ShowFlags.SetDistanceFieldAO(ViewSettings->DistanceFieldAO);
			ShowFlags.SetContactShadows(ViewSettings->ContactShadows);
			ShowFlags.SetEyeAdaptation(ViewSettings->EyeAdaptation);
			ShowFlags.SetBloom(ViewSettings->Bloom);
		}

		if (Options.bOverride_OverrideTimeTo)
		{
			// Turn off time the ultimate source of noise.
			InViewFamily.Time = FGameTime::CreateUndilated(Options.OverrideTimeTo, 0.0f);
		}

		if (Options.bDisableNoisyRenderingFeatures)
	{
			//// Turn off common show flags for noisy sources of rendering.
			//InViewFamily.EngineShowFlags.SetAntiAliasing(false);
			//InViewFamily.EngineShowFlags.SetMotionBlur(false);
			//InViewFamily.EngineShowFlags.SetTemporalAA(false);
			//InViewFamily.EngineShowFlags.SetScreenSpaceReflections(false);
			////InViewFamily.EngineShowFlags.SetScreenSpaceAO(false);
			////InViewFamily.EngineShowFlags.SetDistanceFieldAO(false);
			//InViewFamily.EngineShowFlags.SetContactShadows(false);
			//InViewFamily.EngineShowFlags.SetEyeAdaptation(false);

			//TODO Auto Exposure?
			//TODO EyeAdaptation Gamma?

			// Disable screen percentage.
			//InViewFamily.EngineShowFlags.SetScreenPercentage(false);
		}

		if (Options.bDisableTonemapping)
		{
			//InViewFamily.EngineShowFlags.SetEyeAdaptation(false);
			//InViewFamily.EngineShowFlags.SetTonemapper(false);
		}
	}

	/** We always want to go last. */
	virtual int32 GetPriority() const override { return MIN_int32; }

private:
	FAutomationScreenshotOptions Options;
};

FAutomationTestScreenshotEnvSetup::FAutomationTestScreenshotEnvSetup()
	: DefaultFeature_AntiAliasing(TEXT("r.AntiAliasingMethod"))
	, DefaultFeature_AutoExposure(TEXT("r.DefaultFeature.AutoExposure"))
	, DefaultFeature_MotionBlur(TEXT("r.DefaultFeature.MotionBlur"))
	, MotionBlurQuality(TEXT("r.MotionBlurQuality"))
	, ScreenSpaceReflectionQuality(TEXT("r.SSR.Quality"))
	, EyeAdaptationQuality(TEXT("r.EyeAdaptationQuality"))
	, ContactShadows(TEXT("r.ContactShadows"))
	, TonemapperGamma(TEXT("r.TonemapperGamma"))
	, TonemapperSharpen(TEXT("r.Tonemapper.Sharpen"))
	, ScreenPercentage(TEXT("r.ScreenPercentage"))
	, DynamicResTestScreenPercentage(TEXT("r.DynamicRes.TestScreenPercentage"))
	, DynamicResOperationMode(TEXT("r.DynamicRes.OperationMode"))
	, SecondaryScreenPercentage(TEXT("r.SecondaryScreenPercentage.GameViewport"))
{
}

FAutomationTestScreenshotEnvSetup::~FAutomationTestScreenshotEnvSetup()
{
}

void FAutomationTestScreenshotEnvSetup::Setup(UWorld* InWorld, FAutomationScreenshotOptions& InOutOptions)
{
	check(IsInGameThread());

	WorldPtr = InWorld;

	if (InOutOptions.bDisableNoisyRenderingFeatures)
	{
		DefaultFeature_AntiAliasing.Set(0);
		DefaultFeature_AutoExposure.Set(0);
		DefaultFeature_MotionBlur.Set(0);
		MotionBlurQuality.Set(0);
		ScreenSpaceReflectionQuality.Set(0);
		ContactShadows.Set(0);
		EyeAdaptationQuality.Set(0);
		TonemapperGamma.Set(2.2f);
	}
	else if (InOutOptions.bDisableTonemapping)
	{
		EyeAdaptationQuality.Set(0);
		TonemapperGamma.Set(2.2f);
	}

	// Forces ScreenPercentage=100
	{
		// Completely disable dynamic resolution
		{
			DynamicResTestScreenPercentage.Set(0);
			DynamicResOperationMode.Set(0);

			// Dynamic resolution status change is only taking effect at next dyn res frame.
			GEngine->EmitDynamicResolutionEvent(EDynamicResolutionStateEvent::EndFrame);
			GEngine->EmitDynamicResolutionEvent(EDynamicResolutionStateEvent::BeginFrame);
		}
		ScreenPercentage.Set(100.f);
	}

	// Ignore High-DPI settings
	SecondaryScreenPercentage.Set(100.f);

	InOutOptions.SetToleranceAmounts(InOutOptions.Tolerance);

	const float InCurrentTimeToSimulate = 0.0f;
	AutomationViewExtension = FSceneViewExtensions::NewExtension<FAutomationViewExtension>(InWorld, InOutOptions, InCurrentTimeToSimulate);

	// TODO - I don't like needing to set this here.  Because the gameviewport uses a console variable, it wins.
	if (UGameViewportClient* ViewportClient = AutomationCommon::GetAnyGameViewportClient())
	{
		static IConsoleVariable* ICVar = IConsoleManager::Get().FindConsoleVariable(FBufferVisualizationData::GetVisualizationTargetConsoleCommandName());
		if (ICVar)
		{
			if (ViewportClient->GetEngineShowFlags())
			{
				ViewportClient->GetEngineShowFlags()->SetVisualizeBuffer(InOutOptions.VisualizeBuffer == NAME_None ? false : true);
				ViewportClient->GetEngineShowFlags()->SetTonemapper(InOutOptions.VisualizeBuffer == NAME_None ? true : false);
				ICVar->Set(*InOutOptions.VisualizeBuffer.ToString());
			}
		}
	}
}

void FAutomationTestScreenshotEnvSetup::Restore()
{
	check(IsInGameThread());

	DefaultFeature_AntiAliasing.Restore();
	DefaultFeature_AutoExposure.Restore();
	DefaultFeature_MotionBlur.Restore();
	MotionBlurQuality.Restore();
	ScreenSpaceReflectionQuality.Restore();
	EyeAdaptationQuality.Restore();
	ContactShadows.Restore();
	TonemapperGamma.Restore();
	//TonemapperSharpen.Restore();
	ScreenPercentage.Restore();
	DynamicResOperationMode.Restore();
	DynamicResTestScreenPercentage.Restore();
	SecondaryScreenPercentage.Restore();

	AutomationViewExtension.Reset();

	if (UGameViewportClient* ViewportClient = AutomationCommon::GetAnyGameViewportClient())
	{
		static IConsoleVariable* ICVar = IConsoleManager::Get().FindConsoleVariable(FBufferVisualizationData::GetVisualizationTargetConsoleCommandName());
		if (ICVar)
		{
			if (ViewportClient->GetEngineShowFlags())
			{
				ViewportClient->GetEngineShowFlags()->SetVisualizeBuffer(false);
				ViewportClient->GetEngineShowFlags()->SetTonemapper(true);
				ICVar->Set(TEXT(""));
			}
		}
	}
}

class FAutomationScreenshotTaker
{
public:
	FAutomationScreenshotTaker(UWorld* InWorld, const FString& InScreenShotName, const FString& InNotes, FAutomationScreenshotOptions InOptions)
		: World(InWorld)
		, ScreenShotName(InScreenShotName)
		, Notes(InNotes)
		, Options(InOptions)
		, bNeedsViewportSizeRestore(false)
		, bDeleteQueued(false)
	{
		EnvSetup.Setup(InWorld, Options);

		UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
		if (!FPlatformProperties::HasFixedResolution())
		{
			FSceneViewport* GameViewport = GameViewportClient ? GameViewportClient->GetGameViewport() : nullptr;
			if (GameViewport)
			{
#if WITH_EDITOR
				// In the editor we can only attempt to re-size standalone viewports
				UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);

				const bool bIsPIEViewport = GameViewport->IsPlayInEditorViewport();
				const bool bIsNewViewport = GameViewportClient->GetWorld() && EditorEngine && EditorEngine->WorldIsPIEInNewViewport(GameViewportClient->GetWorld());

				if (!bIsPIEViewport || bIsNewViewport)
#endif
				{
					ViewportRestoreSize = GameViewport->GetSize();
					FIntPoint ScreenshotViewportSize = UAutomationBlueprintFunctionLibrary::GetAutomationScreenshotSize(InOptions);
					GameViewport->SetViewportSize(ScreenshotViewportSize.X, ScreenshotViewportSize.Y);
					bNeedsViewportSizeRestore = true;
				}
			}
		}

		FlushRenderingCommands();

		GameViewportClient->OnScreenshotCaptured().AddRaw(this, &FAutomationScreenshotTaker::GrabScreenShot);
		FWorldDelegates::LevelRemovedFromWorld.AddRaw(this, &FAutomationScreenshotTaker::WorldDestroyed);
		FScreenshotRequest::OnScreenshotRequestProcessed().AddRaw(this, &FAutomationScreenshotTaker::OnScreenshotProcessed);
	}

	virtual ~FAutomationScreenshotTaker()
	{
		FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);
		FScreenshotRequest::OnScreenshotRequestProcessed().RemoveAll(this);

		UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
		if (GameViewportClient)
		{
			// remove before we restore the viewport's size - a resize can trigger a redraw, which would trigger OnScreenshotCaptured() again (endless loop)
			GameViewportClient->OnScreenshotCaptured().RemoveAll(this);
		}

		FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

		if (!FPlatformProperties::HasFixedResolution() && bNeedsViewportSizeRestore)
		{
			if (GameViewportClient)
			{
				FSceneViewport* GameViewport = GameViewportClient->GetGameViewport();
				GameViewport->SetViewportSize(ViewportRestoreSize.X, ViewportRestoreSize.Y);
			}
		}

		EnvSetup.Restore();

		FAutomationTestFramework::Get().NotifyScreenshotTakenAndCompared();
	}

	void DeleteSelfNextFrame()
	{
		if (!bDeleteQueued)
		{
			FTSTicker::GetCoreTicker().AddTicker(TEXT("ScreenshotCleanup"), 0.1f, [this](float) {
				delete this;
				return false;
				});
			bDeleteQueued = true;
		}
	}

	void GrabScreenShot(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData)
	{
		check(IsInGameThread());

		if (World.IsValid())
		{
			FAutomationScreenshotData Data = UAutomationBlueprintFunctionLibrary::BuildScreenshotData(World.Get(), ScreenShotName, InSizeX, InSizeY);

			// Copy the relevant data into the metadata for the screenshot.
			Data.bHasComparisonRules = true;
			Data.ToleranceRed = Options.ToleranceAmount.Red;
			Data.ToleranceGreen = Options.ToleranceAmount.Green;
			Data.ToleranceBlue = Options.ToleranceAmount.Blue;
			Data.ToleranceAlpha = Options.ToleranceAmount.Alpha;
			Data.ToleranceMinBrightness = Options.ToleranceAmount.MinBrightness;
			Data.ToleranceMaxBrightness = Options.ToleranceAmount.MaxBrightness;
			Data.bIgnoreAntiAliasing = Options.bIgnoreAntiAliasing;
			Data.bIgnoreColors = Options.bIgnoreColors;
			Data.MaximumLocalError = Options.MaximumLocalError;
			Data.MaximumGlobalError = Options.MaximumGlobalError;

			// Record any user notes that were made to accompany this shot.
			Data.Notes = Notes;

			bool bAttemptToCompareShot = FAutomationTestFramework::Get().OnScreenshotCaptured().ExecuteIfBound(InImageData, Data);

			UE_LOGF(AutomationFunctionLibrary, Log, "Screenshot captured as %ls", *Data.ScreenshotPath);

			if (GIsAutomationTesting)
			{
				FAutomationTestFramework::Get().OnScreenshotCompared.AddRaw(this, &FAutomationScreenshotTaker::OnComparisonComplete);
				FScreenshotRequest::OnScreenshotRequestProcessed().RemoveAll(this);
				return;
			}
		}

		DeleteSelfNextFrame();
	}

	void OnScreenshotProcessed()
	{
		UE_LOGF(AutomationFunctionLibrary, Log, "Screenshot processed, but not compared.");

		DeleteSelfNextFrame();
	}

	void OnComparisonComplete(const FAutomationScreenshotCompareResults& CompareResults)
	{
		FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);

		if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
		{
			CurrentTest->AddEvent(CompareResults.ToAutomationEvent());
		}

		DeleteSelfNextFrame();
	}

	void WorldDestroyed(ULevel* InLevel, UWorld* InWorld)
	{
		// If the InLevel is null, it's a signal that the entire world is about to disappear, so
		// go ahead and remove this widget from the viewport, it could be holding onto too many
		// dangerous actor references that won't carry over into the next world.
		if (InLevel == nullptr && InWorld == World.Get())
		{
			// we don't delete directly because of the risk of conflicting with an already in flight
			// request to delete ourselves
			World.Reset();
			DeleteSelfNextFrame();
		}
	}

private:

	TWeakObjectPtr<UWorld> World;

	FString	Context;
	FString	ScreenShotName;
	FString Notes;
	FAutomationScreenshotOptions Options;

	FAutomationTestScreenshotEnvSetup EnvSetup;
	FIntPoint ViewportRestoreSize;
	bool bNeedsViewportSizeRestore;
	bool bDeleteQueued;
};

class FAutomationHighResScreenshotGrabber
{
public:
	FAutomationHighResScreenshotGrabber(const FString& InContext, const FString& InScreenShotName, const FString& InNotes, FAutomationScreenshotOptions InOptions)
		: Context(InContext)
		, ScreenShotName(InScreenShotName)
		, Notes(InNotes)
		, Options(InOptions)
	{
		FScreenshotRequest::OnScreenshotCaptured().AddRaw(this, &FAutomationHighResScreenshotGrabber::GrabScreenShot);
		FWorldDelegates::LevelRemovedFromWorld.AddRaw(this, &FAutomationHighResScreenshotGrabber::WorldDestroyed);
	}

	virtual ~FAutomationHighResScreenshotGrabber()
	{
		FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);
		FScreenshotRequest::OnScreenshotCaptured().RemoveAll(this);

		FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

		FAutomationTestFramework::Get().NotifyScreenshotTakenAndCompared();
	}

	void GrabScreenShot(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData)
	{
		FScreenshotRequest::OnScreenshotCaptured().RemoveAll(this);

		FAutomationScreenshotData Data = UAutomationBlueprintFunctionLibrary::BuildScreenshotData(Context, ScreenShotName, InSizeX, InSizeY);

		// Copy the relevant data into the metadata for the screenshot.
		Data.bHasComparisonRules = true;
		Data.ToleranceRed = Options.ToleranceAmount.Red;
		Data.ToleranceGreen = Options.ToleranceAmount.Green;
		Data.ToleranceBlue = Options.ToleranceAmount.Blue;
		Data.ToleranceAlpha = Options.ToleranceAmount.Alpha;
		Data.ToleranceMinBrightness = Options.ToleranceAmount.MinBrightness;
		Data.ToleranceMaxBrightness = Options.ToleranceAmount.MaxBrightness;
		Data.bIgnoreAntiAliasing = Options.bIgnoreAntiAliasing;
		Data.bIgnoreColors = Options.bIgnoreColors;
		Data.MaximumLocalError = Options.MaximumLocalError;
		Data.MaximumGlobalError = Options.MaximumGlobalError;

		// Record any user notes that were made to accompany this shot.
		Data.Notes = Notes;

		bool bAttemptToCompareShot = FAutomationTestFramework::Get().OnScreenshotCaptured().ExecuteIfBound(InImageData, Data);

		UE_LOGF(AutomationFunctionLibrary, Log, "Screenshot captured as %ls", *Data.ScreenshotPath);

		FAutomationTestFramework::Get().OnScreenshotCompared.AddRaw(this, &FAutomationHighResScreenshotGrabber::OnComparisonComplete);
	}

	void OnComparisonComplete(const FAutomationScreenshotCompareResults& CompareResults)
	{
		FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);

		if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
		{
			CurrentTest->AddEvent(CompareResults.ToAutomationEvent());
		}

		delete this;
	}

	void WorldDestroyed(ULevel* InLevel, UWorld* InWorld)
	{
		// If the InLevel is null, it's a signal that the entire world is about to disappear, so
		// go ahead and remove this widget from the viewport, it could be holding onto too many
		// dangerous actor references that won't carry over into the next world.
		if (InLevel == nullptr)
		{
			delete this;
		}
	}

private:
	FString	Context;
	FString	ScreenShotName;
	FString Notes;
	FAutomationScreenshotOptions Options;
};

#endif // WITH_AUTOMATION_TESTS

class FScreenshotTakenState : public FAutomationTaskStatusBase
{
public:
	FScreenshotTakenState(bool InNeedGameViewToggle = true, bool InNeedCameraChange = true)
		: NeedGameViewToggle(InNeedGameViewToggle)
		, NeedCameraChange(InNeedCameraChange)
	{
		if (GIsAutomationTesting)
		{
			// When Automation test are running we hook to the FAutomationTestFramework::OnComparisonComplete instead of the
			// FScreenshotRequest::OnScreenshotRequestProcessed, because with HighResScreenshot, FScreenshotRequest::OnScreenshotRequestProcessed
			// is fired before comparison is completed.
			FAutomationTestFramework::Get().OnScreenshotCompared.AddRaw(this, &FScreenshotTakenState::OnComparisonComplete);
		}
		else
		{
			FScreenshotRequest::OnScreenshotRequestProcessed().AddRaw(this, &FScreenshotTakenState::SetDone);
		}
	};

	virtual ~FScreenshotTakenState()
	{
#if WITH_AUTOMATION_TESTS
		FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);
#endif
		if (!Done)
		{
			FScreenshotRequest::OnScreenshotRequestProcessed().RemoveAll(this);
			UnlockViewport();
		}
	};

	virtual void SetDone() override
	{
		FScreenshotRequest::OnScreenshotRequestProcessed().RemoveAll(this);

		UnlockViewport();

		Done = true;
	};

	void OnComparisonComplete(const FAutomationScreenshotCompareResults& CompareResults)
	{
		FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);
		SetDone();
	};

	void UnlockViewport()
	{
#if WITH_EDITOR
		if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
		{
			FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			SLevelViewport* LevelViewport = LevelEditor.GetFirstActiveLevelViewport().Get();
			if (LevelViewport)
			{
				if (NeedGameViewToggle)
				{
					if (LevelViewport->IsInGameView())
					{
						LevelViewport->ToggleGameView();
					}
					else
					{
						UE_LOGF(AutomationFunctionLibrary, Verbose, "Expected to be able to toggle off the Game View mode after the screenshot was taken, but the Viewport was already no longer in that mode or it is not a Perspective.");
					}
				}
				if (NeedCameraChange)
				{
					FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
					if (LevelViewportClient.IsAnyActorLocked())
					{
						LevelViewportClient.SetActorLock(nullptr);
						LevelViewportClient.bDisableInput = false;
						LevelViewportClient.bEnableFading = true;
					}
				}
			}
		}
#endif
	};

private:
	bool NeedGameViewToggle;
	bool NeedCameraChange;

};

UAutomationBlueprintFunctionLibrary::UAutomationBlueprintFunctionLibrary(const class FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

void UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot()
{
	FlushAsyncLoading();

	UWorld* CurrentWorld{ nullptr };
	// Make sure we finish all level streaming
	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		if (UWorld* GameWorld = GameEngine->GetGameWorld())
		{
			CurrentWorld = GameWorld;
			GameWorld->FlushLevelStreaming(EFlushLevelStreamingType::Full);
		}
	}

	// Finish compiling the shaders if the platform doesn't require cooked data.
	if (!FPlatformProperties::RequiresCookedData())
	{
		UMaterialInterface::SubmitRemainingJobsForWorld(CurrentWorld);
		FAssetCompilingManager::Get().FinishAllCompilation();
		IAutomationControllerModule* AutomationControllerModule = FModuleManager::GetModulePtr<IAutomationControllerModule>("AutomationController");
		if (AutomationControllerModule != nullptr)
		{
			AutomationControllerModule->GetAutomationController()->ResetAutomationTestTimeout(TEXT("shader compilation"));
		}
	}

	// Force all mip maps to load before taking the screenshot.
	UTexture::ForceUpdateTextureStreaming();

	IStreamingManager::Get().StreamAllResources(0.0f);
}

FIntPoint UAutomationBlueprintFunctionLibrary::GetAutomationScreenshotSize(const FAutomationScreenshotOptions& Options)
{
	// Fallback resolution if all else fails for screenshots.
	uint32 ResolutionX = 1280;
	uint32 ResolutionY = 720;

	// First get the default set for the project.
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	if (AutomationTestSettings->DefaultScreenshotResolution.GetMin() > 0)
	{
		ResolutionX = (uint32)AutomationTestSettings->DefaultScreenshotResolution.X;
		ResolutionY = (uint32)AutomationTestSettings->DefaultScreenshotResolution.Y;
	}

	// If there's an override resolution, use that instead.
	if (Options.Resolution.GetMin() > 0)
	{
		ResolutionX = (uint32)Options.Resolution.X;
		ResolutionY = (uint32)Options.Resolution.Y;
	}
	else
	{
		// Failing to find an override, look for a platform override that may have been provided through the
		// device profiles setup, to configure the CVars for controlling the automation screenshot size.
		int32 OverrideWidth = CVarAutomationScreenshotResolutionWidth.GetValueOnGameThread();
		int32 OverrideHeight = CVarAutomationScreenshotResolutionHeight.GetValueOnGameThread();

		if (OverrideWidth > 0)
		{
			ResolutionX = (uint32)OverrideWidth;
		}

		if (OverrideHeight > 0)
		{
			ResolutionY = (uint32)OverrideHeight;
		}
	}

	return FIntPoint(ResolutionX, ResolutionY);
}

FAutomationScreenshotData UAutomationBlueprintFunctionLibrary::BuildScreenshotData(const FString& MapOrContext, const FString& ScreenShotName, int32 Width, int32 Height)
{
	FString TestName = TEXT("");
	if (FAutomationTestFramework::Get().GetCurrentTest()) 
	{
		TestName = FAutomationTestFramework::Get().GetCurrentTest()->GetTestFullName();
	}

#if WITH_AUTOMATION_TESTS
	FAutomationScreenshotData Data = AutomationCommon::BuildScreenshotData(MapOrContext, TestName, ScreenShotName, Width, Height);
	return Data;
#else
	return FAutomationScreenshotData();
#endif
}

FAutomationScreenshotData UAutomationBlueprintFunctionLibrary::BuildScreenshotData(UWorld* InWorld, const FString& ScreenShotName, int32 Width, int32 Height)
{
	return BuildScreenshotData(AutomationCommon::GetWorldContext(InWorld), ScreenShotName, Width, Height);
}

bool UAutomationBlueprintFunctionLibrary::TakeAutomationScreenshotInternal(UObject* WorldContextObject, const FString& ScreenShotName, const FString& Notes, FAutomationScreenshotOptions Options)
{
	UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot();

#if WITH_AUTOMATION_TESTS
	FAutomationScreenshotTaker* TempObject = new FAutomationScreenshotTaker(WorldContextObject ? WorldContextObject->GetWorld() : nullptr, ScreenShotName, Notes, Options);
#endif

	FScreenshotRequest::RequestScreenshot(false);
	return true; //-V773
}

void UAutomationBlueprintFunctionLibrary::TakeAutomationScreenshot(UObject* WorldContextObject, FLatentActionInfo LatentInfo, const FString& InScreenShotName, const FString& Notes, const FAutomationScreenshotOptions& Options)
{
	if ( GIsAutomationTesting )
	{
		FString ScreenShotName = InScreenShotName;
		if ( ScreenShotName.IsEmpty() )
		{
			ScreenShotName = TEXT("Undefined");
			UE_LOGF(AutomationFunctionLibrary, Warning, "Screenshot name is empty. Default name will be used.");
		}
		if ( UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) )
		{
			FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
			if ( LatentActionManager.FindExistingAction<FTakeScreenshotAfterTimeLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr )
			{
				LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FTakeScreenshotAfterTimeLatentAction(LatentInfo, ScreenShotName, Notes, Options));
			}
		}
	}
	else
	{
		UE_LOGF(AutomationFunctionLibrary, Log, "Screenshot not captured - screenshots are only taken during automation tests");
	}
}

void UAutomationBlueprintFunctionLibrary::TakeAutomationScreenshotAtCamera(UObject* WorldContextObject, FLatentActionInfo LatentInfo, ACameraActor* Camera, const FString& NameOverride, const FString& Notes, const FAutomationScreenshotOptions& Options)
{
	if ( Camera == nullptr )
	{
		FMessageLog("PIE").Error(LOCTEXT("CameraRequired", "A camera is required to TakeAutomationScreenshotAtCamera"));
		return;
	}

	UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
	if (GameViewportClient == nullptr)
	{
		FMessageLog("PIE").Error(LOCTEXT("GameViewportRequired", "No game viewport found in World to TakeAutomationScreenshotAtCamera. Use a delay or change Net mode to Standalone."));
		return;
	}

	APlayerController* PlayerController = UGameplayStatics::GetPlayerController(GameViewportClient->GetWorld(), 0);
	if ( PlayerController == nullptr )
	{
		FMessageLog("PIE").Error(LOCTEXT("PlayerRequired", "A player controller is required to TakeAutomationScreenshotAtCamera"));
		return;
	}

	// Move the player, then queue up a screenshot.
	// We need to delay before the screenshot so that the motion blur has time to stop.
	PlayerController->bAutoManageActiveCameraTarget = false;
	PlayerController->SetViewTarget(Camera, FViewTargetTransitionParams());
	FString ScreenshotName = Camera->GetName();

	if ( !NameOverride.IsEmpty() )
	{
		ScreenshotName = NameOverride;
	}

	if ( UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull) )
	{
		ScreenshotName = FString::Printf(TEXT("%s_%s"), *World->GetName(), *ScreenshotName);

		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if ( LatentActionManager.FindExistingAction<FTakeScreenshotAfterTimeLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr )
		{
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FTakeScreenshotAfterTimeLatentAction(LatentInfo, ScreenshotName, Notes, Options));
		}
	}
}

bool UAutomationBlueprintFunctionLibrary::TakeAutomationScreenshotOfUI_Immediate(UObject* WorldContextObject, const FString& ScreenShotName, const FAutomationScreenshotOptions& Options)
{
	UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot();

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (UGameViewportClient* GameViewport = WorldContextObject->GetWorld()->GetGameViewport())
		{
			TSharedPtr<SViewport> Viewport = GameViewport->GetGameViewportWidget();
			if (Viewport.IsValid())
			{
				TArray<FColor> OutColorData;
				FIntVector OutSize;
				if (FSlateApplication::Get().TakeScreenshot(Viewport.ToSharedRef(), OutColorData, OutSize))
				{
#if WITH_AUTOMATION_TESTS
					// For UI, we only care about what the final image looks like. So don't compare alpha channel.
					// In editor, scene is rendered into a PF_B8G8R8A8 RT and then copied over to the R10B10G10A2 swapchain back buffer and
					// this copy ignores alpha. In game, however, scene is directly rendered into the back buffer and the alpha values are
					// already meaningless at that stage.
					for (int32 Idx = 0; Idx < OutColorData.Num(); ++Idx)
					{
						OutColorData[Idx].A = 0xff;
					}

					// The screenshot taker deletes itself later.
					FAutomationScreenshotTaker* TempObject = new FAutomationScreenshotTaker(World, ScreenShotName, TEXT(""), Options);

					FAutomationScreenshotData Data = BuildScreenshotData(World->GetName(), ScreenShotName, OutSize.X, OutSize.Y);

					// Copy the relevant data into the metadata for the screenshot.
					Data.bHasComparisonRules = true;
					Data.ToleranceRed = Options.ToleranceAmount.Red;
					Data.ToleranceGreen = Options.ToleranceAmount.Green;
					Data.ToleranceBlue = Options.ToleranceAmount.Blue;
					Data.ToleranceAlpha = Options.ToleranceAmount.Alpha;
					Data.ToleranceMinBrightness = Options.ToleranceAmount.MinBrightness;
					Data.ToleranceMaxBrightness = Options.ToleranceAmount.MaxBrightness;
					Data.bIgnoreAntiAliasing = Options.bIgnoreAntiAliasing;
					Data.bIgnoreColors = Options.bIgnoreColors;
					Data.MaximumLocalError = Options.MaximumLocalError;
					Data.MaximumGlobalError = Options.MaximumGlobalError;

					if (UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient())
					{
						GameViewportClient->OnScreenshotCaptured().Broadcast(OutSize.X, OutSize.Y, OutColorData);
					}
#endif

					return true;
				}
			}
		}
	}

	return false;
}

void UAutomationBlueprintFunctionLibrary::TakeAutomationScreenshotOfUI(UObject* WorldContextObject, FLatentActionInfo LatentInfo, const FString& Name, const FAutomationScreenshotOptions& Options)
{
	if (TakeAutomationScreenshotOfUI_Immediate(WorldContextObject, Name, Options))
	{
		FLatentActionManager& LatentActionManager = WorldContextObject->GetWorld()->GetLatentActionManager();
		if ( LatentActionManager.FindExistingAction<FTakeScreenshotAfterTimeLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr )
		{
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FWaitForScreenshotComparisonLatentAction(LatentInfo));
		}
	}
}

void UAutomationBlueprintFunctionLibrary::EnableStatGroup(UObject* WorldContextObject, FName GroupName)
{
#if STATS
	if (FGameThreadStatsData* StatsData = FLatestGameThreadStatsData::Get().Latest)
	{
		const FString GroupNameString = FString(TEXT("STATGROUP_")) + GroupName.ToString();
		const FName GroupNameFull = FName(*GroupNameString, EFindName::FNAME_Find);
		if(StatsData->GroupNames.Contains(GroupNameFull))
		{
			return;
		}
	}

	if (APlayerController* TargetPC = UGameplayStatics::GetPlayerController(WorldContextObject, 0))
	{
		TargetPC->ConsoleCommand( FString(TEXT("stat ")) + GroupName.ToString() + FString(TEXT(" -nodisplay")), /*bWriteToLog=*/false);
	}
#endif
}

void UAutomationBlueprintFunctionLibrary::DisableStatGroup(UObject* WorldContextObject, FName GroupName)
{
#if STATS
	if (FGameThreadStatsData* StatsData = FLatestGameThreadStatsData::Get().Latest)
	{
		const FString GroupNameString = FString(TEXT("STATGROUP_")) + GroupName.ToString();
		const FName GroupNameFull = FName(*GroupNameString, EFindName::FNAME_Find);

		if (!StatsData->GroupNames.Contains(GroupNameFull))
		{
			return;
		}
	}

	if (APlayerController* TargetPC = UGameplayStatics::GetPlayerController(WorldContextObject, 0))
	{
		TargetPC->ConsoleCommand(FString(TEXT("stat ")) + GroupName.ToString() + FString(TEXT(" -nodisplay")), /*bWriteToLog=*/false);
	}
#endif
}

#if STATS
template <EComplexStatField::Type ValueType, bool bCallCount = false>
float HelperGetStat(FName StatName)
{
	if (FGameThreadStatsData* StatsData = FLatestGameThreadStatsData::Get().Latest)
	{
		if (const FComplexStatMessage* StatMessage = StatsData->GetStatData(StatName))
		{
			if(bCallCount)
			{
				return (float)StatMessage->GetValue_CallCount(ValueType);
			}
			else
			{
				return (float)FPlatformTime::ToMilliseconds64(StatMessage->GetValue_Duration(ValueType));
			}
		}
	}

#if WITH_EDITOR
	FText WarningOut = FText::Format(LOCTEXT("StatNotFound", "Could not find stat data for {0}, did you call ToggleStatGroup with enough time to capture data?"), FText::FromName(StatName));
	FMessageLog("PIE").Warning(WarningOut);
	UE_LOGF(AutomationFunctionLibrary, Warning, "%ls", *WarningOut.ToString());
#endif

	return 0.f;
}
#endif

float UAutomationBlueprintFunctionLibrary::GetStatIncAverage(FName StatName)
{
#if STATS
	return HelperGetStat<EComplexStatField::IncAve>(StatName);
#else
	return 0.0f;
#endif
}

float UAutomationBlueprintFunctionLibrary::GetStatIncMax(FName StatName)
{
#if STATS
	return HelperGetStat<EComplexStatField::IncMax>(StatName);
#else
	return 0.0f;
#endif
}

float UAutomationBlueprintFunctionLibrary::GetStatExcAverage(FName StatName)
{
#if STATS
	return HelperGetStat<EComplexStatField::ExcAve>(StatName);
#else
	return 0.0f;
#endif
}

float UAutomationBlueprintFunctionLibrary::GetStatExcMax(FName StatName)
{
#if STATS
	return HelperGetStat<EComplexStatField::ExcMax>(StatName);
#else
	return 0.0f;
#endif
}

float UAutomationBlueprintFunctionLibrary::GetStatCallCount(FName StatName)
{
#if STATS
	return HelperGetStat<EComplexStatField::IncAve, /*bCallCount=*/true>(StatName);
#else
	return 0.0f;
#endif
}

bool UAutomationBlueprintFunctionLibrary::AreAutomatedTestsRunning()
{
	return GIsAutomationTesting;
}

class FWaitForLoadingToFinish : public FPendingLatentAction
{
public:
	FWaitForLoadingToFinish(const FLatentActionInfo& LatentInfo, UObject* InWorldContextObject, const FAutomationWaitForLoadingOptions& InOptions)
		: ExecutionFunction(LatentInfo.ExecutionFunction)
		, OutputLink(LatentInfo.Linkage)
		, CallbackTarget(LatentInfo.CallbackTarget)
		, WorldPtr(InWorldContextObject ? InWorldContextObject->GetWorld() : nullptr)
		, Options(InOptions)
	{
		UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot();

		WaitingFrames = 0;
		LastLoadTime = FPlatformTime::Seconds();

		if (Options.WaitForReplicationToSettle)
		{
			if (UWorld* MyWorld = GetWorld())
			{
				FOnActorSpawned::FDelegate ActorSpawnedDelegate = FOnActorSpawned::FDelegate::CreateRaw(this, &FWaitForLoadingToFinish::OnActorSpawned);
				ActorSpawnedDelegateHandle = MyWorld->AddOnActorSpawnedHandler(ActorSpawnedDelegate);
			}
		}
	}

	virtual ~FWaitForLoadingToFinish()
	{
		if (UWorld* MyWorld = GetWorld())
		{
			MyWorld->RemoveOnActorSpawnedHandler(ActorSpawnedDelegateHandle);
		}
	}

	UWorld* GetWorld()
	{
		if (UWorld* World = WorldPtr.Get())
		{
			return World;
		}
		else
		{
			if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
			{
				return GameEngine->GetGameWorld();
			}
		}

		return nullptr;
	}

	void OnActorSpawned(AActor* SpawnedActor)
	{
		if (SpawnedActor->GetLocalRole() != ROLE_Authority)
		{
			bActorReplicationDetected = true;
		}
	}

	bool AnyLevelStreaming()
	{
		// Make sure we finish all level streaming
		if (UWorld* World = GetWorld())
		{
			for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
			{
				// See whether there's a level with a pending request.
				if (LevelStreaming)
				{
					if (LevelStreaming->HasLoadRequestPending())
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		bool bResetWaiting = false;

		if (IsAsyncLoading())
		{
			bResetWaiting = true;
		}
		else if (AnyLevelStreaming())
		{
			bResetWaiting = true;
		}
		else if (bActorReplicationDetected)
		{
			bResetWaiting = true;
			bActorReplicationDetected = false;
		}

		if (bResetWaiting)
		{
			WaitingFrames = 0;
			LastLoadTime = FPlatformTime::Seconds();
		}
		else
		{
			WaitingFrames++;
		}

		const double WaitingTime = (FPlatformTime::Seconds() - LastLoadTime);

		// Needs to have been both 60 frames, and at least 5 seconds from the last load event.
		if (WaitingFrames > 60 && WaitingTime > 5)
		{
			Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
		}
	}

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override
	{
		return TEXT("Waiting For Loading");
	}
#endif

private:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;

	TWeakObjectPtr<UWorld> WorldPtr;

	FDelegateHandle ActorSpawnedDelegateHandle;

	FAutomationWaitForLoadingOptions Options;

	int32 WaitingFrames = 0;
	double LastLoadTime = 0;

	bool bActorReplicationDetected = false;
};


void UAutomationBlueprintFunctionLibrary::AutomationWaitForLoading(UObject* WorldContextObject, FLatentActionInfo LatentInfo, FAutomationWaitForLoadingOptions Options)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
		if (LatentActionManager.FindExistingAction<FWaitForLoadingToFinish>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FWaitForLoadingToFinish(LatentInfo, WorldContextObject, Options));
		}
	}
}

UAutomationEditorTask* UAutomationBlueprintFunctionLibrary::TakeHighResScreenshot(int32 ResX, int32 ResY, FString Filename, ACameraActor* Camera, bool bMaskEnabled, bool bCaptureHDR, EComparisonTolerance ComparisonTolerance, FString ComparisonNotes, float Delay, bool bForceGameView)
{
	UAutomationEditorTask* Task = NewObject<UAutomationEditorTask>();
	FGCObjectScopeGuard TaskGuard(Task);

#if WITH_EDITOR
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		if (uint32(ResX) <= GetMax2DTextureDimension() && uint32(ResY) <= GetMax2DTextureDimension())
		{
			FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			SLevelViewport* LevelViewport = LevelEditor.GetFirstActiveLevelViewport().Get();
			bool bNeedGameViewToggle = bForceGameView && !LevelViewport->IsInGameView();
			if (bNeedGameViewToggle)
			{
				LevelViewport->ToggleGameView();
			}

			// Move Viewport to Camera
			bool bNeedCameraChange = Camera != nullptr;
			if (bNeedCameraChange)
			{
				FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
				// We set the actor lock (pilot mode) and force the viewport to match the camera now.
				// We unset the actor lock later when the screenshot is done. See FScreenshotTakenState.SetDone().
				LevelViewportClient.SetActorLock(Camera);
				LevelViewportClient.UpdateViewForLockedActor();
				LevelViewportClient.bDisableInput = true;
				LevelViewportClient.bEnableFading = false;
			}

			FinishLoadingBeforeScreenshot();

			Task->BindTask(MakeUnique<FScreenshotTakenState>(bNeedGameViewToggle, bNeedCameraChange));

			// Delay taking the screenshot by a few frames
			FTSTicker::GetCoreTicker().AddTicker(TEXT("ScreenshotDelay"), Delay, [LevelViewport, ComparisonTolerance, ComparisonNotes, Filename, ResX, ResY, bMaskEnabled, bCaptureHDR](float) {
					FHighResScreenshotConfig& HighResScreenshotConfig = GetHighResScreenshotConfig();
					HighResScreenshotConfig.SetResolution(ResX, ResY);
					HighResScreenshotConfig.SetFilename(Filename);
					HighResScreenshotConfig.SetMaskEnabled(bMaskEnabled);
					HighResScreenshotConfig.SetHDRCapture(bCaptureHDR);

					LevelViewport->GetActiveViewport()->TakeHighResScreenShot();
#if WITH_AUTOMATION_TESTS
					if (GIsAutomationTesting)
					{
						if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
						{
							FString Context = CurrentTest->GetTestContext();
							if (Context.IsEmpty()) { Context = CurrentTest->GetTestName(); }
							FAutomationScreenshotOptions ComparisonOptions = FAutomationScreenshotOptions(ComparisonTolerance);
							FAutomationHighResScreenshotGrabber* TempObject = new FAutomationHighResScreenshotGrabber(Context, Filename, ComparisonNotes, ComparisonOptions);
						} //-V773
					}
#endif
					return false;
				}
			);

			return Task;
		}

		UE_LOGF(AutomationFunctionLibrary, Error, "Screenshot size exceeds the maximum allowed texture size (%d x %d)", GetMax2DTextureDimension(), GetMax2DTextureDimension());
	}
#endif
	return Task;
}

bool UAutomationBlueprintFunctionLibrary::CompareImageAgainstReference(FString InImagePath, FString ComparisonName, EComparisonTolerance InTolerance, FString InNotes, UObject* WorldContextObject)
{
#if WITH_AUTOMATION_TESTS
	if (GIsAutomationTesting)
	{
		const FString ImageExtension = FPaths::GetExtension(InImagePath);
		const EImageFormat ImageFormat = ImageWrapperHelper::GetImageFormat(ImageExtension);

		IImageWrapperModule& ImageWrapperModule = FModuleManager::GetModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
		TSharedPtr<IImageWrapper> ImageReader = ImageWrapperModule.CreateImageWrapper(ImageFormat);
		if (!ImageReader.IsValid())
		{
			UE_LOGF(AutomationFunctionLibrary, Error, "Unable to locate image processor for %ls file format", *ImageExtension);
			return false;
		}

		TArray64<uint8> ImageData;
		const bool OpenSuccess = FFileHelper::LoadFileToArray(ImageData, *InImagePath);

		if (!OpenSuccess)
		{
			UE_LOGF(AutomationFunctionLibrary, Error, "Unable to read image %ls", *InImagePath);
			return false;
		}

		if (!ImageReader->SetCompressed(ImageData.GetData(), ImageData.Num()))
		{
			UE_LOGF(AutomationFunctionLibrary, Error, "Unable to parse image %ls", *InImagePath);
			return false;
		}

		if (ImageReader->GetBitDepth() != 8)
		{
			UE_LOGF(AutomationFunctionLibrary, Error, "Automation can only compare 8bit depth channel. %ls has %dbit per channel.", *InImagePath, ImageReader->GetBitDepth());
			return false;
		}

		const int32 Width = ImageReader->GetWidth();
		const int32 Height = ImageReader->GetHeight();
		TArray<FColor> ImageDataDecompressed;
		ImageDataDecompressed.SetNum(Width * Height);

		if (!ImageReader->GetRaw(ERGBFormat::BGRA, 8, TArrayView64<uint8>((uint8*)ImageDataDecompressed.GetData(), ImageDataDecompressed.Num() * 4)))
		{
			UE_LOGF(AutomationFunctionLibrary, Error, "Unable to decompress image %ls", *InImagePath);
			return false;
		}

		if (ComparisonName.IsEmpty())
		{
			ComparisonName = FPaths::GetBaseFilename(InImagePath);
		}

		FString Context = TEXT("");
		if (FFunctionalTestBase::IsFunctionalTestRunning() && WorldContextObject != nullptr)
		{
			// Functional tests have a different rule to name their test, mainly because part of the full test name is a path.
			// So, to keep name short and still comprehensible, we are going to use the map name + the actor label instead.
			UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
			Context = World != nullptr ? World->GetName() : TEXT("UnknownMap");
			Context += TEXT(".") + FFunctionalTestBase::GetRunningTestName();
		}

		RequestImageComparison(ComparisonName, Width, Height, ImageDataDecompressed, (EAutomationComparisonToleranceLevel)InTolerance, Context, InNotes);

		return true;
	}
#endif
	UE_LOGF(AutomationFunctionLibrary, Warning, "Can compare image only during test automation.");
	return false;
}

void UAutomationBlueprintFunctionLibrary::AddTestTelemetryData(FString DataPoint, float Measurement, FString Context)
{
	if (GIsAutomationTesting)
	{
		if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
		{
			CurrentTest->AddTelemetryData(DataPoint, Measurement, Context);
		}
	}
}

void UAutomationBlueprintFunctionLibrary::SetTestTelemetryStorage(FString StorageName)
{
	if (GIsAutomationTesting)
	{
		if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
		{
			CurrentTest->SetTelemetryStorage(StorageName);
		}
	}
}

FAutomationScreenshotOptions UAutomationBlueprintFunctionLibrary::GetDefaultScreenshotOptionsForGameplay(EComparisonTolerance Tolerance, float Delay)
{
	FAutomationScreenshotOptions Options;
	Options.Delay = Delay;
	Options.Tolerance = Tolerance;
	Options.bDisableNoisyRenderingFeatures = true;
	Options.bIgnoreAntiAliasing = true;
	Options.SetToleranceAmounts(Tolerance);

	return Options;
}

FAutomationScreenshotOptions UAutomationBlueprintFunctionLibrary::GetDefaultScreenshotOptionsForRendering(EComparisonTolerance Tolerance, float Delay)
{
	FAutomationScreenshotOptions Options;
	Options.Delay = Delay;
	Options.Tolerance = Tolerance;
	Options.bDisableNoisyRenderingFeatures = true;
	Options.bIgnoreAntiAliasing = true;
	Options.SetToleranceAmounts(Tolerance);

	return Options;
}

void UAutomationBlueprintFunctionLibrary::AddExpectedLogError(FString ExpectedPatternString, int32 Occurrences, bool ExactMatch, bool IsRegex)
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddExpectedError(ExpectedPatternString, ExactMatch? EAutomationExpectedErrorFlags::Exact:EAutomationExpectedErrorFlags::Contains, Occurrences, IsRegex);
	}
}

void UAutomationBlueprintFunctionLibrary::AddExpectedPlainLogError(FString ExpectedString, int32 Occurrences, bool ExactMatch)
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddExpectedErrorPlain(ExpectedString, ExactMatch ? EAutomationExpectedErrorFlags::Exact : EAutomationExpectedErrorFlags::Contains, Occurrences);
	}
}

void UAutomationBlueprintFunctionLibrary::AddExpectedLogMessage(FString ExpectedPatternString, int32 Occurrences, bool ExactMatch, bool IsRegex)
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddExpectedMessage(ExpectedPatternString, ExactMatch ? EAutomationExpectedErrorFlags::Exact : EAutomationExpectedErrorFlags::Contains, Occurrences, IsRegex);
	}
}

void UAutomationBlueprintFunctionLibrary::AddExpectedPlainLogMessage(FString ExpectedString, int32 Occurrences, bool ExactMatch)
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddExpectedMessagePlain(ExpectedString, ExactMatch ? EAutomationExpectedErrorFlags::Exact : EAutomationExpectedErrorFlags::Contains, Occurrences);
	}
}

void UAutomationBlueprintFunctionLibrary::SetScalabilityQualityLevelRelativeToMax(UObject* WorldContextObject, int32 Value /*= 1*/)
{
	Scalability::FQualityLevels Quality;
	Quality.SetFromSingleQualityLevelRelativeToMax(Value);
	Scalability::SetQualityLevels(Quality, true);
}

void UAutomationBlueprintFunctionLibrary::SetScalabilityQualityToEpic(UObject* WorldContextObject)
{
	Scalability::FQualityLevels Quality;
	Quality.SetFromSingleQualityLevelRelativeToMax(0);
	Scalability::SetQualityLevels(Quality, true);
}

void UAutomationBlueprintFunctionLibrary::SetScalabilityQualityToLow(UObject* WorldContextObject)
{
	Scalability::FQualityLevels Quality;
	Quality.SetFromSingleQualityLevel(0);
	Scalability::SetQualityLevels(Quality, true);
}

void UAutomationBlueprintFunctionLibrary::SetEditorViewportViewMode(EViewModeIndex Index)
{
#if WITH_EDITOR
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
		{
			if (LevelViewport.IsValid())
			{
				if (TSharedPtr<FEditorViewportClient> Viewport = LevelViewport->GetViewportClient())
				{
					Viewport->SetViewMode(Index);
				}
			}
		}
	}
#endif
}

void UAutomationBlueprintFunctionLibrary::SetEditorActiveViewportViewMode(EViewModeIndex Index)
{
#if WITH_EDITOR
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (SLevelViewport* LevelViewport = LevelEditorModule.GetFirstActiveLevelViewport().Get())
	{
		if (TSharedPtr<FEditorViewportClient> Viewport = LevelViewport->GetViewportClient())
		{
			Viewport->SetViewMode(Index);
		}
	}
#endif
}

EViewModeIndex UAutomationBlueprintFunctionLibrary::GetEditorActiveViewportViewMode()
{
#if WITH_EDITOR
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (SLevelViewport* LevelViewport = LevelEditorModule.GetFirstActiveLevelViewport().Get())
	{
		if (TSharedPtr<FEditorViewportClient> Viewport = LevelViewport->GetViewportClient())
		{
			return Viewport->GetViewMode();
		}
	}
#endif
	return EViewModeIndex::VMI_Unknown;
}

void UAutomationBlueprintFunctionLibrary::SetEditorActiveViewportWireframeOpacity(float Opacity)
{
#if WITH_EDITOR
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (SLevelViewport* LevelViewport = LevelEditorModule.GetFirstActiveLevelViewport().Get())
	{
		if (TSharedPtr<FEditorViewportClient> Viewport = LevelViewport->GetViewportClient())
		{
			Viewport->WireframeOpacity = Opacity;
		}
	}
#endif
}

float UAutomationBlueprintFunctionLibrary::GetEditorActiveViewportWireframeOpacity()
{
#if WITH_EDITOR
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (SLevelViewport* LevelViewport = LevelEditorModule.GetFirstActiveLevelViewport().Get())
	{
		if (TSharedPtr<FEditorViewportClient> Viewport = LevelViewport->GetViewportClient())
		{
			return Viewport->WireframeOpacity;
		}
	}
#endif
	return 0.0f;
}

void UAutomationBlueprintFunctionLibrary::SetEditorViewportVisualizeBuffer( FName BufferName )
{
#if WITH_EDITOR
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
		{
			if (LevelViewport.IsValid())
			{
				if (TSharedPtr<FEditorViewportClient> Viewport = LevelViewport->GetViewportClient())
				{
					Viewport->ChangeBufferVisualizationMode(BufferName);
				}
			}
		}
	}
#endif
}

void UAutomationBlueprintFunctionLibrary::AddTestInfo(const FString& InLogItem)
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddInfo(InLogItem);
	}
}

void UAutomationBlueprintFunctionLibrary::AddTestWarning(const FString& InLogItem)
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddWarning(InLogItem);
	}
}

void UAutomationBlueprintFunctionLibrary::AddTestError(const FString& InLogItem)
{
	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddError(InLogItem);
	}
}

#undef LOCTEXT_NAMESPACE

