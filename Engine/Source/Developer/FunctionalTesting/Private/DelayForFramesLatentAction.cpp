// Copyright Epic Games, Inc. All Rights Reserved.
#include "DelayForFramesLatentAction.h"
#include "Engine/LatentActionManager.h"

FDelayForFramesLatentAction::FDelayForFramesLatentAction(const FLatentActionInfo& LatentInfo, int32 NumFrames)
	: ExecutionFunction(LatentInfo.ExecutionFunction)
	, OutputLink(LatentInfo.Linkage)
	, CallbackTarget(LatentInfo.CallbackTarget)
	, FramesRemaining(NumFrames)
{
}

FDelayForFramesLatentAction::~FDelayForFramesLatentAction()
{
}

void FDelayForFramesLatentAction::UpdateOperation(FLatentResponse& Response)
{
	--FramesRemaining;
	Response.FinishAndTriggerIf(FramesRemaining <= 0, ExecutionFunction, OutputLink, CallbackTarget);
}

#if WITH_EDITOR
FString FDelayForFramesLatentAction::GetDescription() const
{
	return FString::Printf(TEXT("Delay (%d frames remaining)"), FramesRemaining);
}
#endif
