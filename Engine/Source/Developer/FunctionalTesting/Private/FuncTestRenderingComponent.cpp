// Copyright Epic Games, Inc. All Rights Reserved.

#include "FuncTestRenderingComponent.h"
#include "MeshElementCollector.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/CollisionProfile.h"
#include "FunctionalTest.h"
#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FuncTestRenderingComponent)

//----------------------------------------------------------------------//
// FFTestRenderingSceneProxy
//----------------------------------------------------------------------//
class FFTestRenderingSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	/** Initialization constructor. */
	FFTestRenderingSceneProxy(const UFuncTestRenderingComponent& InComponent)
		: FPrimitiveSceneProxy(&InComponent)
	{
		OwningTest = Cast<AFunctionalTest>(InComponent.GetOwner());
		if (OwningTest)
		{
			TArray<AActor*> RelevantActors;
			OwningTest->GatherRelevantActors(RelevantActors);
			
			TestActorLocation = OwningTest->GetActorLocation();
			for (auto Actor : RelevantActors)
			{
				if (Actor)
				{
					Locations.Add(Actor->GetActorLocation());
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View) && IsSelected();
		Result.bDynamicRelevance = true;
		// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
		Result.bSeparateTranslucency = Result.bNormalTranslucency = IsShown(View);
		return Result;
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		static const FLinearColor RadiusColor = FLinearColor(FColorList::Orange);
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

				for (auto Location : Locations)
				{
					PDI->DrawLine(TestActorLocation, Location, RadiusColor, SDPG_World, 3.f);
					DrawArrowHead(PDI, Location, TestActorLocation, 30.f, RadiusColor, SDPG_World, 3.f);
				}
			}
		}
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	SIZE_T GetAllocatedSize( void ) const { return FPrimitiveSceneProxy::GetAllocatedSize() + Locations.GetAllocatedSize(); }

protected:
	AFunctionalTest* OwningTest;
	FVector TestActorLocation;
	TArray<FVector> Locations;
};
//----------------------------------------------------------------------//
// UFuncTestRenderingComponent
//----------------------------------------------------------------------//
UFuncTestRenderingComponent::UFuncTestRenderingComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Allows updating in game, while optimizing rendering for the case that it is not modified
	Mobility = EComponentMobility::Stationary;

	BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	bIsEditorOnly = true;

	SetGenerateOverlapEvents(false);
}

FPrimitiveSceneProxy* UFuncTestRenderingComponent::CreateSceneProxy()
{
	return new FFTestRenderingSceneProxy(*this);
}

FBoxSphereBounds UFuncTestRenderingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const AFunctionalTest* Owner = Cast<AFunctionalTest>(GetOwner());
	if (Owner)
	{
		FBox BoundingBox(ForceInitToZero);

		TArray<AActor*> RelevantActors;
		Owner->GatherRelevantActors(RelevantActors);

		BoundingBox += Owner->GetActorLocation();
		for (auto Actor : RelevantActors)
		{
			if (Actor)
			{
				BoundingBox += Actor->GetActorLocation();
			}
		}

		return FBoxSphereBounds(BoundingBox);
	}

	return FBoxSphereBounds(ForceInitToZero);
}

