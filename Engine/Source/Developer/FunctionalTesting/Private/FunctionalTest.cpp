// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalTest.h"
#include "FunctionalTestingModule.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "GameFramework/Pawn.h"
#include "Misc/Paths.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LatentActionManager.h"
#include "Engine/Level.h"
#include "Components/BillboardComponent.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Misc/AutomationTest.h"
#include "GameFramework/PlayerController.h"
#include "Components/TextRenderComponent.h"
#include "Engine/Selection.h"
#include "FuncTestRenderingComponent.h"
#include "ObjectEditorUtils.h"
#include "VisualLogger/VisualLogger.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "DelayForFramesLatentAction.h"
#include "Engine/DebugCameraController.h"
#include "TraceQueryTestResults.h"
#include "Misc/RuntimeErrors.h"
#include "FunctionalTestBase.h"
#include "UnrealClient.h"
#include "RenderTimer.h"

#if WITH_EDITOR
#include "AutomationControllerSettings.h"
#include "ISessionFrontendModule.h"
#include "Misc/MessageDialog.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(FunctionalTest)

DECLARE_CYCLE_STAT(TEXT("FunctionalTest - RunTest"), STAT_FunctionalTest_RunTest, STATGROUP_FunctionalTest);
DECLARE_CYCLE_STAT(TEXT("FunctionalTest - StartTest"), STAT_FunctionalTest_StartTest, STATGROUP_FunctionalTest);
DECLARE_CYCLE_STAT(TEXT("FunctionalTest - PrepareTest"), STAT_FunctionalTest_PrepareTest, STATGROUP_FunctionalTest);
DECLARE_CYCLE_STAT(TEXT("FunctionalTest - Tick"), STAT_FunctionalTest_TickTest, STATGROUP_FunctionalTest);
DECLARE_CYCLE_STAT(TEXT("FunctionalTest - FinishTest"), STAT_FunctionalTest_FinishTest, STATGROUP_FunctionalTest);

namespace
{
	template <typename T>
	bool PerformComparison(const T& lhs, const T& rhs, EComparisonMethod comparison)
	{
		switch (comparison)
		{
		case EComparisonMethod::Equal_To:
			return lhs == rhs;

		case EComparisonMethod::Not_Equal_To:
			return lhs != rhs;

		case EComparisonMethod::Greater_Than_Or_Equal_To:
			return lhs >= rhs;

		case EComparisonMethod::Less_Than_Or_Equal_To:
			return lhs <= rhs;

		case EComparisonMethod::Greater_Than:
			return lhs > rhs;

		case EComparisonMethod::Less_Than:
			return lhs < rhs;
		}

		UE_LOGF(LogFunctionalTest, Error, "Invalid comparison method");
		return false;
	}

	FString GetComparisonAsString(EComparisonMethod comparison)
	{
		UEnum* Enum = StaticEnum<EComparisonMethod>();
		return Enum->GetNameStringByValue((uint8)comparison).ToLower().Replace(TEXT("_"), TEXT(" "), ESearchCase::CaseSensitive);
	}

	FString TransformToString(const FTransform &transform)
	{
		const FRotator R(transform.Rotator());
		FVector T(transform.GetTranslation());
		FVector S(transform.GetScale3D());

		return FString::Printf(TEXT("Translation: %f, %f, %f | Rotation: %f, %f, %f | Scale: %f, %f, %f"), T.X, T.Y, T.Z, R.Pitch, R.Yaw, R.Roll, S.X, S.Y, S.Z);
	}

	void DelayForFramesCommon(UObject* WorldContextObject, FLatentActionInfo LatentInfo, int32 NumFrames)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
			if (LatentActionManager.FindExistingAction<FDelayForFramesLatentAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
			{
				LatentActionManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, new FDelayForFramesLatentAction(LatentInfo, NumFrames));
			}
		}
	}
}

/* Return a readable string of the provided EFunctionalTestResult enum */
FString LexToString(const EFunctionalTestResult TestResult)
{
	switch (TestResult)
	{
		case EFunctionalTestResult::Default:
			return FString("Default");
		case EFunctionalTestResult::Invalid:
			return FString("Invalid");
		case EFunctionalTestResult::Error:
			return FString("Error");
		case EFunctionalTestResult::Running:
			return FString("Running");
		case EFunctionalTestResult::Failed:
			return FString("Failed");
		case EFunctionalTestResult::Succeeded:
			return FString("Succeeded");
	}
	return FString("Unhandled EFunctionalTestResult Enum!");
}


FString MapPackageToAutomationPath(const FString& MapPackageName)
{
	FString PartialSuiteName;
	if (MapPackageName.StartsWith(TEXT("/Game/")))
	{
		PartialSuiteName = MapPackageName.RightChop(6); // Remove "/Game/" from the name, it is not descriptive
	}
	else
	{
		PartialSuiteName = MapPackageName.RightChop(1); // Remove leading slash
	}
	return PartialSuiteName.Replace(TEXT("/"), TEXT(".")); // use dot syntax
}


FString MapPackageToAutomationPath(const FAssetData& MapAsset)
{
	static const FName TestPathOverrideName = "TestPathOverride";

	FString StoredPathName;
	if (MapAsset.GetTagValue(TestPathOverrideName, StoredPathName))
	{
		return StoredPathName;
	}

	return MapPackageToAutomationPath(MapAsset.PackageName.ToString());
}


AFunctionalTest::AFunctionalTest( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, TestLabel(GetName())
	, bIsEnabled(true)
	, bIsInSublevel(false)
	, PersistentLevelName()
	, LogErrorHandling(EFunctionalTestLogHandling::ProjectDefault)
	, LogWarningHandling(EFunctionalTestLogHandling::ProjectDefault)
	, bShouldDelayGarbageCollection(true)
	, Result(EFunctionalTestResult::Invalid)
	, PreparationTimeLimit(15.0f)
	, TimeLimit(60.0f)
	, TimesUpMessage( NSLOCTEXT("FunctionalTest", "DefaultTimesUpMessage", "Time's Up.") )
	, TimesUpResult(EFunctionalTestResult::Failed)
	, bIsRunning(false)
	, TotalTime(0.f)
	, RunFrame(0)
	, RunTime(0.0f)
	, StartFrame(0)
	, StartTime(0.0f)
	, bIsReady(false)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = false;
	PrimaryActorTick.bTickEvenWhenPaused = true;
	
	SetCanBeDamaged(false);
	bEnableAutoLODGeneration = false;

	SpriteComponent = CreateDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (SpriteComponent)
	{
		SpriteComponent->bHiddenInGame = true;
#if WITH_EDITORONLY_DATA

		if (!IsRunningCommandlet())
		{
			struct FConstructorStatics
			{
				ConstructorHelpers::FObjectFinderOptional<UTexture2D> Texture;
				FName ID_FTests;
				FText NAME_FTests;

				FConstructorStatics()
					: Texture(TEXT("/Engine/EditorResources/S_FTest"))
					, ID_FTests(TEXT("FTests"))
					, NAME_FTests(NSLOCTEXT( "SpriteCategory", "FTests", "FTests" ))
				{
				}
			};
			static FConstructorStatics ConstructorStatics;

			SpriteComponent->Sprite = ConstructorStatics.Texture.Get();
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_FTests;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_FTests;
		}

#endif
		RootComponent = SpriteComponent;
	}

#if WITH_EDITORONLY_DATA
	RenderComp = CreateDefaultSubobject<UFuncTestRenderingComponent>(TEXT("RenderComp"));
	RenderComp->SetupAttachment(RootComponent);
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	static bool bSelectionHandlerSetUp = false;
	if (HasAnyFlags(RF_ClassDefaultObject) && !HasAnyFlags(RF_TagGarbageTemp) && bSelectionHandlerSetUp == false)
	{
		USelection::SelectObjectEvent.AddStatic(&AFunctionalTest::OnSelectObject);
		bSelectionHandlerSetUp = true;
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	TestName = CreateEditorOnlyDefaultSubobject<UTextRenderComponent>(TEXT("TestName"));
	if ( TestName )
	{
		TestName->bHiddenInGame = true;
		TestName->SetHorizontalAlignment(EHTA_Center);
		TestName->SetRelativeLocation(FVector(0, 0, 80));
		TestName->SetRelativeRotation(FRotator(0, 0, 0));
		TestName->SetupAttachment(RootComponent);
	}

	bIsSpatiallyLoaded = false;
#endif
}

#if WITH_EDITOR
void AFunctionalTest::EditTags()
{
	UAutomationControllerSettings* Settings = UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();
	if (Settings)
	{
		Settings->InitialSelectedTestsNames.Empty();
		Settings->InitialSelectedTestsNames.Add(GetBeautifiedName());
		Settings->bOpenTagEditorWhenInitialSelectedTestsNames = true;
	}

	ISessionFrontendModule& SessionFrontend = FModuleManager::LoadModuleChecked<ISessionFrontendModule>("SessionFrontend");
	SessionFrontend.InvokeSessionFrontend(FName("AutomationPanel"));
}
#endif

void AFunctionalTest::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

#if WITH_EDITOR
	TestLabel = GetActorLabel();

	if ( TestName )
	{
		if ( bIsEnabled )
		{
			TestName->SetTextRenderColor(FColor(45, 255, 0));
			TestName->SetText(FText::FromString(GetActorLabel()));
		}
		else
		{
			TestName->SetTextRenderColor(FColor(55, 55, 55));
			TestName->SetText(FText::FromString(GetActorLabel() + TEXT("\n") + TEXT("# Disabled #")));
		}

		//TestName->SetTextMaterial();
	}
#endif
}

bool AFunctionalTest::RunTest(const TArray<FString>& Params)
{
	SCOPE_CYCLE_COUNTER(STAT_FunctionalTest_RunTest);
	UWorld* World = GetWorld();
	#if WITH_EDITOR
	const bool bIsEditorOnlyObject = IsEditorOnlyObject(this);
	const bool bIsEditorOnlyLoadedInPIE = IsEditorOnlyLoadedInPIE();

	ensure(World->HasBegunPlay() || (World->WorldType == EWorldType::Editor && bIsEditorOnlyObject && !bIsEditorOnlyLoadedInPIE));

	if (bIsEditorOnlyObject && !bIsEditorOnlyLoadedInPIE)
	{
		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FunctionalTestCanRunInEditorWorld)
		{
			AddWarning(FString::Printf(TEXT("%s changed behavior, return true in IsEditorOnlyLoadedInPIE to run in PIE world, otherwise resave the asset to acknowledge behavior change"), *GetName()));
		}
	}
	#else
	ensure(World->HasBegunPlay());
	#endif
	
	FFunctionalTestBase* FunctionalTest = static_cast<FFunctionalTestBase*>(FAutomationTestFramework::Get().GetCurrentTest());

	// Set handling of warnings/errors based on this test. Tests can either specify an explicit option or choose to go with the
	// project defaults.
	TOptional<bool> bSuppressErrors, bSuppressWarnings, bWarningsAreErrors;

	if (LogErrorHandling != EFunctionalTestLogHandling::ProjectDefault)
	{
		bSuppressErrors = LogErrorHandling == EFunctionalTestLogHandling::OutputIgnored ? true : false;
	}

	if (LogWarningHandling != EFunctionalTestLogHandling::ProjectDefault)
	{
		// warnings can be set to be suppressed, or elevated to errors
		bSuppressWarnings = LogWarningHandling == EFunctionalTestLogHandling::OutputIgnored ? true : false;
		bWarningsAreErrors = LogWarningHandling == EFunctionalTestLogHandling::OutputIsError;
	}

	if (FunctionalTest)
	{
		FunctionalTest->SetLogErrorAndWarningHandling(bSuppressErrors, bSuppressWarnings, bWarningsAreErrors);
		FunctionalTest->SetFunctionalTestRunning(TestLabel);
		if (FAutomationTestFramework::NeedLogBPTestMetadata() && GIsAutomationTesting)
		{
			AddInfo(FString::Printf(TEXT("[Owner] %s"), *Author));
			AddInfo(FString::Printf(TEXT("[Description] %s"), *Description));
			AddInfo(FString::Printf(TEXT("[TestTags] %s"), *TestTags));
		}
	}

	FailureMessage = TEXT("");
	
	//Do not collect garbage during the test. We force GC at the end.
	//GEngine->DelayGarbageCollection();

	RunFrame = GFrameNumber;
	RunTime = (float)World->GetTimeSeconds();

	TotalTime = 0.f;
	if (TimeLimit >= 0)
	{
		SetActorTickEnabled(true);
	}

	bIsReady = false;
	bIsRunning = true;

	GoToObservationPoint();

	PrepareTest();
	OnTestPrepare.Broadcast();

	return true;
}

void AFunctionalTest::PrepareTest()
{
	SCOPE_CYCLE_COUNTER(STAT_FunctionalTest_PrepareTest);
	ReceivePrepareTest();
}

void AFunctionalTest::StartTest()
{
	SCOPE_CYCLE_COUNTER(STAT_FunctionalTest_StartTest);
	TotalTime = 0.f;
	StartFrame = GFrameNumber;
	StartTime = (float)GetWorld()->GetTimeSeconds();

	ReceiveStartTest();
	OnTestStart.Broadcast();
}

void AFunctionalTest::OnTimeout()
{
	FText FailureReason;

	if (bIsReady)
	{
		FailureReason = FText::Format(NSLOCTEXT("FunctionalTest", "TimeOutInTest", "{0}. Test timed out in {1} seconds"),
			TimesUpMessage, FText::AsNumber(TotalTime));
	}
	else
	{
		FailureReason = FText::Format(NSLOCTEXT("FunctionalTest", "TimeOutInTestPrep", "{0}. Test preparation timed out in {1} seconds"),
			TimesUpMessage, FText::AsNumber(TotalTime));
	}

	FinishTest(TimesUpResult, FailureReason.ToString());
}

void AFunctionalTest::Tick(float DeltaSeconds)
{
	// already requested not to tick. 
	if ( bIsRunning == false )
	{
		return;
	}
	SCOPE_CYCLE_COUNTER(STAT_FunctionalTest_TickTest);
	
	//Allow Functional Tests to configure if GC is delayed until the end. 
	if (bShouldDelayGarbageCollection)
	{
		//Do not collect garbage during the test. We force GC at the end.
		GEngine->DelayGarbageCollection();
	}

	TotalTime += DeltaSeconds;

	if ( !bIsReady )
	{
		bIsReady = IsReady();

		// Once we're finally ready to begin the test, then execute the Start event.
		if ( bIsReady )
		{
			StartTest();
		}
		else
		{
			if (PreparationTimeLimit > 0.f && TotalTime > PreparationTimeLimit)
			{
				OnTimeout();
			}
		}
	}
	else
	{
		if (TimeLimit > 0.f && TotalTime > TimeLimit)
		{
			OnTimeout();
		}
	}

	Super::Tick(DeltaSeconds);
}

bool AFunctionalTest::IsReady_Implementation()
{
	return true;
}

void AFunctionalTest::FinishTest(EFunctionalTestResult TestResult, const FString& Message)
{
	if (bIsRunning == false)
	{
		// ignore
		return;
	}
	SCOPE_CYCLE_COUNTER(STAT_FunctionalTest_FinishTest);
	
	// Do reporting first. When we start cleaning things up internal states that capture results
	// are reset.
	Result = TestResult;
	
	switch (TestResult)
	{
		case EFunctionalTestResult::Invalid:
		case EFunctionalTestResult::Error:
		case EFunctionalTestResult::Failed:
			AddError(FString::Printf(TEXT("FinishTest TestResult=%s. %s"), *LexToString(TestResult), *Message));
			break;
			
		case EFunctionalTestResult::Running:
			AddWarning(FString::Printf(TEXT("FinishTest TestResult=%s. %s"), *LexToString(TestResult), *Message));
			break;
			
		default:
			if (!Message.IsEmpty())
			{
				LogStep(ELogVerbosity::Log, *Message);
			}
			break;
	}
	
	if (FFunctionalTestBase* FunctionalTest = static_cast<FFunctionalTestBase*>(FAutomationTestFramework::Get().GetCurrentTest()))
	{
		FunctionalTest->SetFunctionalTestComplete(TestLabel);
	}

	bIsRunning = false;
	SetActorTickEnabled(false);

	ReceiveTestFinished();
	OnTestFinished.Broadcast();

	TObjectPtr<AActor>* ActorToDestroy = AutoDestroyActors.GetData();

	for (int32 ActorIndex = 0; ActorIndex < AutoDestroyActors.Num(); ++ActorIndex, ++ActorToDestroy)
	{
		if (*ActorToDestroy != NULL)
		{
			// will be removed next frame
			(*ActorToDestroy)->SetLifeSpan( 0.01f );
		}
	}
	
	AutoDestroyActors.Reset();
	
	//Force GC at the end of every test.
	GEngine->ForceGarbageCollection();
	
	//if (AdditionalDetails.IsEmpty() == false)
	//{
	//	const FString AdditionalDetails = FString::Printf(TEXT("%s %s, time %.2fs"), *GetAdditionalTestFinishedMessage(TestResult), *OnAdditionalTestFinishedMessageRequest(TestResult), TotalTime);
	//	UE_LOGF(LogFunctionalTest, Log, "%ls", *AdditionalDetails);
	//}

	TestFinishedObserver.ExecuteIfBound(this);

	EnvSetup = nullptr;
}

void AFunctionalTest::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// If end play occurs and we're still running, notify that the testing has stopped.
	if (bIsRunning)
	{
		// Tell the test it is being aborted
		FinishTest(EFunctionalTestResult::Invalid, TEXT("Test was aborted"));
	}

	TestFinishedObserver.Unbind();

	Super::EndPlay(EndPlayReason);
}

void AFunctionalTest::CleanUp()
{
	FailureMessage = TEXT("");
}

bool AFunctionalTest::IsRunning() const
{
	return bIsRunning;
}

bool AFunctionalTest::IsEnabled() const
{
	return bIsEnabled;
}

bool AFunctionalTest::IsEnabledInWorld(const UWorld* World) const
{
	if (bIsInSublevel)
	{
		return World != nullptr && World->GetFName() == PersistentLevelName
			? bIsEnabled
			: false;
	}

	return bIsEnabled;
}

//@todo add "warning" level here
void AFunctionalTest::LogMessage(const FString& Message)
{
	UE_LOGF(LogFunctionalTest, Log, "%ls", *Message);
	UE_VLOG(this, LogFunctionalTest, Log
		, TEXT("%s> %s")
		, *TestLabel, *Message);
}

void AFunctionalTest::SetTimeLimit(float InTimeLimit, EFunctionalTestResult InResult)
{
	if (InTimeLimit < 0.f)
	{
		UE_VLOG(this, LogFunctionalTest, Warning
			, TEXT("%s> Trying to set TimeLimit to less than 0. Falling back to 0 (infinite).")
			, *TestLabel);

		InTimeLimit = 0.f;
	}
	TimeLimit = InTimeLimit;

	if (InResult == EFunctionalTestResult::Invalid)
	{
		UE_VLOG(this, LogFunctionalTest, Warning
			, TEXT("%s> Trying to set test Result to \'Invalid\'. Falling back to \'Failed\'")
			, *TestLabel);

		InResult = EFunctionalTestResult::Failed;
	}
	TimesUpResult = InResult;
}

void AFunctionalTest::GatherRelevantActors(TArray<AActor*>& OutActors) const
{
	if (ObservationPoint)
	{
		OutActors.AddUnique(ObservationPoint);
	}

	for (auto Actor : AutoDestroyActors)
	{
		if (Actor)
		{
			OutActors.AddUnique(Actor);
		}
	}

	OutActors.Append(DebugGatherRelevantActors());
}

void AFunctionalTest::AddRerun(FName Reason)
{
	RerunCauses.Add(Reason);
}

FName AFunctionalTest::GetCurrentRerunReason()const
{
	return CurrentRerunCause;
}

void AFunctionalTest::SetConsoleVariable(const FString& Name, const FString& InValue)
{
	// Initialize our Scoped Test Environment to be used during the duration of the test
	if (!EnvSetup.IsValid())
	{
		EnvSetup = FScopedTestEnvironment::Get();
	}
	EnvSetup->SetConsoleVariableValue(Name, InValue);
}

void AFunctionalTest::SetConsoleVariableFromInteger(const FString& Name, const int32 InValue)
{
	SetConsoleVariable(Name, FString::FromInt(InValue));
}

void AFunctionalTest::SetConsoleVariableFromFloat(const FString& Name, const float InValue)
{
	SetConsoleVariable(Name, FString::SanitizeFloat(InValue));
}

void AFunctionalTest::SetConsoleVariableFromBoolean(const FString& Name, const bool InValue)
{
	SetConsoleVariable(Name, FString::FromInt(InValue));
}

void AFunctionalTest::RegisterAutoDestroyActor(AActor* ActorToAutoDestroy)
{
	AutoDestroyActors.AddUnique(ActorToAutoDestroy);
}

#if WITH_EDITOR

void AFunctionalTest::PostLoad()
{
	Super::PostLoad();

	SetFlags(RF_Transactional);

	PreviousTestTags = TestTags;
}

void AFunctionalTest::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_FunctionalTesting = FName(TEXT("Functional Testing"));
	static const FName NAME_TimeLimit = FName(TEXT("TimeLimit"));
	static const FName NAME_TimesUpResult = FName(TEXT("TimesUpResult"));
	static const FName NAME_TestTags = FName(TEXT("TestTags"));

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != NULL)
	{
		if (FObjectEditorUtils::GetCategoryFName(PropertyChangedEvent.Property) == NAME_FunctionalTesting)
		{
			// first validate new data since there are some dependencies
			if (PropertyChangedEvent.Property->GetFName() == NAME_TimeLimit)
			{
				if (TimeLimit < 0.f)
				{
					TimeLimit = 0.f;
				}
			}
			else if (PropertyChangedEvent.Property->GetFName() == NAME_TimesUpResult)
			{
				if (TimesUpResult == EFunctionalTestResult::Invalid)
				{
					TimesUpResult = EFunctionalTestResult::Failed;
				}
			}
			else if (PropertyChangedEvent.Property->GetFName() == NAME_TestTags && !TestTags.IsEmpty())
			{
				bool bContinueEdit = true;
				// this property will be deprected
				if (EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(TEXT("Editing asset tags are being deprecated and \"Edit Test Tags\" should be used instead. Continue?"))))
				{
					PreviousTestTags = TestTags;
				}
				else
				{
					bContinueEdit = false;
					TestTags = PreviousTestTags;
				}

				if (bContinueEdit)
				{
					const FString FullTestName = GetBeautifiedName();
					if (!FullTestName.IsEmpty())
					{
						FAutomationTestFramework& TestFramework = FAutomationTestFramework::Get();
						TestFramework.UnregisterAutomationTestTags(FullTestName);
						TestFramework.RegisterAutomationTestTags(FullTestName, TestTags);
					}
				}
			}
		}
	}
}

FString AFunctionalTest::GetBeautifiedName()
{
	if (const ULevel* Level = GetLevel())
	{
		if (const UPackage* Package = Level->GetPackage())
		{
			FSoftObjectPath CurrentMapPath = FSoftObjectPath(this->GetLevel()->GetPackage());
			FAssetData MapAsset = IAssetRegistry::Get()->GetAssetByObjectPath(CurrentMapPath);
			const FString PartialSuiteName = MapPackageToAutomationPath(MapAsset);
			const FString FullBeautifiedName(PartialSuiteName + TEXT(".") + *GetActorLabel());
			const FString FullTestName(TEXT("Project.Functional Tests." + FullBeautifiedName));

			return FullTestName;
		}
	}

	return TEXT("");
}

void AFunctionalTest::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (IsPackageExternal() && IsEnabled())
	{
		const FString TestActor = GetActorLabel() + TEXT("|") + GetName() + TEXT("|") + TestTags;
		const TCHAR* TestCategory = IsEditorOnlyObject(this) ? TEXT("TestNameEditor") : TEXT("TestName");
		Context.AddTag(UObject::FAssetRegistryTag(TestCategory, TestActor, UObject::FAssetRegistryTag::TT_Hidden));
	}
}

void AFunctionalTest::OnSelectObject(UObject* NewSelection)
{
	AFunctionalTest* AsFTest = Cast<AFunctionalTest>(NewSelection);
	if (AsFTest)
	{
		AsFTest->MarkComponentsRenderStateDirty();
	}
}

#endif // WITH_EDITOR

void AFunctionalTest::GoToObservationPoint()
{
	if (ObservationPoint == nullptr)
	{
		return;
	}

	UWorld* World = GetWorld();
	if (World && World->GetGameInstance())
	{
		APlayerController* TargetPC = nullptr;
		for (FConstPlayerControllerIterator PCIterator = World->GetPlayerControllerIterator(); PCIterator; ++PCIterator)
		{
			APlayerController* PC = PCIterator->Get();

			// Don't use debug camera player controllers.
			// While it's tempting to teleport the camera if the user is debugging something then moving the camera around will them.
			if (PC && !PC->IsA(ADebugCameraController::StaticClass()))
			{
				TargetPC = PC;
				break;
			}
		}

		if (TargetPC)
		{
			if (TargetPC->GetPawn())
			{
				TargetPC->GetPawn()->TeleportTo(ObservationPoint->GetActorLocation(), ObservationPoint->GetActorRotation(), /*bIsATest=*/false, /*bNoCheck=*/true);
				TargetPC->SetControlRotation(ObservationPoint->GetActorRotation());
			}
			else
			{
				TargetPC->SetViewTarget(ObservationPoint);
			}
		}
	}
}

/** Returns SpriteComponent subobject **/
UBillboardComponent* AFunctionalTest::GetSpriteComponent()
{
	return SpriteComponent;
}

//////////////////////////////////////////////////////////////////////////

bool AFunctionalTest::AssertTrue(bool Condition, const FString& Message, const UObject* ContextObject)
{
	if ( !Condition )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Assertion in Blueprint failed: '%s' for context '%s'"), *Message, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Assertion passed (%s)"), *Message));
		return true;
	}
}

bool AFunctionalTest::AssertFalse(bool Condition, const FString& Message, const UObject* ContextObject)
{
	return AssertTrue(!Condition, Message, ContextObject);
}

bool AFunctionalTest::AssertIsValid(UObject* Object, const FString& Message, const UObject* ContextObject)
{
	if ( !IsValid(Object) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Invalid object: '%s' for context '%s'"), *Message, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Valid object: (%s)"), *Message));
		return true;
	}
}

bool AFunctionalTest::AssertValue_Int(int32 Actual, EComparisonMethod ShouldBe, int32 Expected, const FString& What, const UObject* ContextObject)
{
	if ( !PerformComparison(Actual, Expected, ShouldBe) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("%s: expected {%d} to be %s {%d} for context '%s'"), *What, Actual, *GetComparisonAsString(ShouldBe), Expected, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("%s: expected {%d} to be %s {%d} for context '%s'"), *What, Actual, *GetComparisonAsString(ShouldBe), Expected, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return true;
	}
}

bool AFunctionalTest::AssertValue_Float(float Actual, EComparisonMethod ShouldBe, float Expected, const FString& What, const UObject* ContextObject)
{
	if ( !PerformComparison(Actual, Expected, ShouldBe) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("%s: expected {%f} to be %s {%f} for context '%s'"), *What, Actual, *GetComparisonAsString(ShouldBe), Expected, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("%s: expected {%f} to be %s {%f} for context '%s'"), *What, Actual, *GetComparisonAsString(ShouldBe), Expected, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return true;
	}
}

bool AFunctionalTest::AssertValue_Double(double Actual, EComparisonMethod ShouldBe, double Expected, const FString& What, const UObject* ContextObject)
{
	if ( !PerformComparison(Actual, Expected, ShouldBe) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("%s: expected {%lf} to be %s {%lf} for context '%s'"), *What, Actual, *GetComparisonAsString(ShouldBe), Expected, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("%s: expected {%lf} to be %s {%lf} for context '%s'"), *What, Actual, *GetComparisonAsString(ShouldBe), Expected, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return true;
	}
}

bool AFunctionalTest::AssertValue_DateTime(FDateTime Actual, EComparisonMethod ShouldBe, FDateTime Expected, const FString& What, const UObject* ContextObject)
{
	if ( !PerformComparison(Actual, Expected, ShouldBe) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("%s: expected {%s} to be %s {%s} for context '%s'"), *What, *Actual.ToString(), *GetComparisonAsString(ShouldBe), *Expected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("DateTime assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Float(const float Actual, const float Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if ( !FMath::IsNearlyEqual(Actual, Expected, Tolerance) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%f}, but it was {%f} within tolerance {%f} for context '%s'"), *What, Expected, Actual, Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Float assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Double(const double Actual, const double Expected, const FString& What, const double Tolerance, const UObject* ContextObject)
{
	if ( !FMath::IsNearlyEqual(Actual, Expected, Tolerance) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%lf}, but it was {%lf} within tolerance {%lf} for context '%s'"), *What, Expected, Actual, Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Double assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Bool(const bool Actual, const bool Expected, const FString& What, const UObject* ContextObject)
{
	if (Actual != Expected)
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%d}, but it was {%d} for context '%s'"), *What, Expected, Actual, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Bool assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Int(const int32 Actual, const int32 Expected, const FString& What, const UObject* ContextObject)
{
	if (Actual != Expected)
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%d}, but it was {%d} for context '%s'"), *What, Expected, Actual, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Bool assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Name(const FName Actual, const FName Expected, const FString& What, const UObject* ContextObject)
{
	if (Actual != Expected)
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s}, but it was {%s} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("FName assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Object(UObject* Actual, UObject* Expected, const FString& What, const UObject* ContextObject)
{
	if (Actual != Expected)
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s}, but it was {%s} for context '%s'"), *What, *GetNameSafe(Expected), *GetNameSafe(Actual), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Object assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Transform(const FTransform& Actual, const FTransform& Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if ( !Expected.Equals(Actual, Tolerance) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s}, but it was {%s} within tolerance {%f} for context '%s'"), *What, *TransformToString(Expected), *TransformToString(Actual), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Transform assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Transform(const FTransform& Actual, const FTransform& NotExpected, const FString& What, const UObject* ContextObject)
{
	if ( NotExpected.Equals(Actual) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *TransformToString(NotExpected), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Transform assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Rotator(FRotator Actual, FRotator Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if ( !Expected.Equals(Actual, Tolerance) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Rotator assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_RotatorOrientation(FRotator Actual, FRotator Expected, const FString& What, float Tolerance, const UObject* ContextObject)
{
	if ( !Expected.EqualsOrientation(Actual, Tolerance) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Rotator assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Rotator(FRotator Actual, FRotator NotExpected, const FString& What, const UObject* ContextObject)
{
	if ( NotExpected.Equals(Actual) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Rotator assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Vector(FVector Actual, FVector Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if ( !Expected.Equals(Actual, Tolerance) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Vector assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Vector(FVector Actual, FVector NotExpected, const FString& What, const UObject* ContextObject)
{
	if ( NotExpected.Equals(Actual) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Vector assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Vector2D(FVector2D Actual, FVector2D Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Vector2D assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Vector2D(FVector2D Actual, FVector2D NotExpected, const FString& What, const UObject* ContextObject)
{
	if (NotExpected.Equals(Actual))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Vector2D assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Box2D(FBox2D Actual, FBox2D Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Vector2D assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Box2D(FBox2D Actual, FBox2D NotExpected, const FString& What, const UObject* ContextObject)
{
	if (NotExpected.Equals(Actual))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Vector2D assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Vector4(FVector4 Actual, FVector4 Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Vector4 assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Vector4(FVector4 Actual, FVector4 NotExpected, const FString& What, const UObject* ContextObject)
{
	if (NotExpected.Equals(Actual))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Vector4 assertion passed (%s)"), *What));
		return true;
	}
}


bool AFunctionalTest::AssertEqual_Plane(FPlane Actual, FPlane Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Plane assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Plane(FPlane Actual, FPlane NotExpected, const FString& What, const UObject* ContextObject)
{
	if (NotExpected.Equals(Actual))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Plane assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Quat(FQuat Actual, FQuat Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Quat assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Quat(FQuat Actual, FQuat NotExpected, const FString& What, const UObject* ContextObject)
{
	if (NotExpected.Equals(Actual))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Quat assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_Matrix(FMatrix Actual, FMatrix Expected, const FString& What, const float Tolerance, const UObject* ContextObject)
{
	if (!Expected.Equals(Actual, Tolerance))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} within tolerance {%f} for context '%s'"), *What, *Expected.ToString(), *Actual.ToString(), Tolerance, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Matrix assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_Matrix(FMatrix Actual, FMatrix NotExpected, const FString& What, const UObject* ContextObject)
{
	if (NotExpected.Equals(Actual))
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("Matrix assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_String(FString Actual, FString Expected, const FString& What, const UObject* ContextObject)
{
	if ( !Expected.Equals(Actual) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} for context '%s'"), *What, *Expected, *Actual, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("String assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertNotEqual_String(FString Actual, FString NotExpected, const FString& What, const UObject* ContextObject)
{
	if ( NotExpected.Equals(Actual) )
	{
		LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' not to be {%s} for context '%s'"), *What, *NotExpected, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		LogStep(ELogVerbosity::Log, FString::Printf(TEXT("String assertion passed (%s)"), *What));
		return true;
	}
}

bool AFunctionalTest::AssertEqual_TraceQueryResults(const UTraceQueryTestResults* Actual, const UTraceQueryTestResults* Expected, const FString& What, const UObject* ContextObject)
{
	return Actual->AssertEqual(Expected, What, ContextObject, *this);
}

void AFunctionalTest::AddWarning(const FString& Message)
{
	LogStep(ELogVerbosity::Warning, Message);
}

void AFunctionalTest::AddError(const FString& Message)
{
	LogStep(ELogVerbosity::Error, Message);
}

void AFunctionalTest::AddInfo(const FString& Message)
{
	LogStep(ELogVerbosity::Log, Message);
}

void AFunctionalTest::LogStep(ELogVerbosity::Type Verbosity, const FString& Message)
{
	TStringBuilder<256> FullMessage;

	FullMessage.Append(TestLabel);
	FullMessage.Append(TEXT(": "));
	FullMessage.Append(Message);

	if ( IsInStep() )
	{
		FullMessage.Append(TEXT(" in step: "));
		FString StepName = GetCurrentStepName();
		if ( StepName.IsEmpty() )
		{
			StepName = TEXT("<UN-NAMED STEP>");
		}
		FullMessage.Append(StepName);
	}

	const int32 STACK_OFFSET = 2;
	FFunctionalTestBase* CurrentFunctionalTest = static_cast<FFunctionalTestBase*>(FAutomationTestFramework::Get().GetCurrentTest());

	// Warn if we do not have a current functional test. Such a situation prevents Warning/Error results from being associated with an actual test
	if (!CurrentFunctionalTest)
	{
		UE_LOGF(LogFunctionalTest, Warning, "FunctionalTest '%ls' ran test '%ls' when no functional test was active. This result will not be tracked.", *TestLabel, *Message);
	}

	/* 
		Note - unlike FAutomationTestOutputDevice::Serialize logging we do not downgrade/suppress logging levels based on the properties of the functional test 
		actor or the project. While AFunctionalTest uses the verbosity enums these messages are  added directly by the test 
		// (e.g. via AddWarning, AddError, Assert_Equal) so they are not considered side-effect warnings/errors	that may be optionally ignored.
	*/

	switch (Verbosity)
	{
	case ELogVerbosity::Log:
		if (CurrentFunctionalTest)
		{
			CurrentFunctionalTest->AddInfo(*FullMessage, STACK_OFFSET);
		}
		else
		{
			UE_VLOG(this, LogFunctionalTest, Log, TEXT("%s"), *FullMessage);
			UE_LOGF(LogFunctionalTest, Log, "%ls", *FullMessage);
		}
		break;

	case ELogVerbosity::Display:
		if (CurrentFunctionalTest)
		{
			CurrentFunctionalTest->AddInfo(*FullMessage, STACK_OFFSET);
		}
		else
		{
			UE_VLOG(this, LogFunctionalTest, Display, TEXT("%s"), *FullMessage);
			UE_LOGF(LogFunctionalTest, Display, "%ls", *FullMessage);
		}
		break;

	case ELogVerbosity::Warning:
		if (CurrentFunctionalTest)
		{
			CurrentFunctionalTest->AddWarning(*FullMessage, STACK_OFFSET);
		}
		else
		{
			UE_VLOG(this, LogFunctionalTest, Warning, TEXT("%s"), *FullMessage);
			UE_LOGF(LogFunctionalTest, Warning, "%ls", *FullMessage);
		}
		break;

	case ELogVerbosity::Error:
		if (CurrentFunctionalTest)
		{
			CurrentFunctionalTest->AddError(*FullMessage, STACK_OFFSET);
		}
		else
		{
			UE_VLOG(this, LogFunctionalTest, Error, TEXT("%s"), *FullMessage);
			UE_LOGF(LogFunctionalTest, Error, "%ls", *FullMessage);
		}
		break;
	}
}

FString	AFunctionalTest::GetCurrentStepName() const
{
	return IsInStep() ? Steps.Top() : FString();
}

void AFunctionalTest::StartStep(const FString& StepName)
{
	Steps.Push(StepName);
}

void AFunctionalTest::FinishStep()
{
	if ( Steps.Num() > 0 )
	{
		Steps.Pop();
	}
	else
	{
		AddWarning(TEXT("FinishStep was called when no steps were currently in progress."));
	}
}

bool AFunctionalTest::IsInStep() const
{
	return Steps.Num() > 0;
}

//////////////////////////////////////////////////////////////////////////

FPerfStatsRecord::FPerfStatsRecord(FString InName)
: Name(InName)
, GPUBudget(0.0f)
, RenderThreadBudget(0.0f)
, GameThreadBudget(0.0f)
{
}

void FPerfStatsRecord::SetBudgets(float InGPUBudget, float InRenderThreadBudget, float InGameThreadBudget)
{
	GPUBudget = InGPUBudget;
	RenderThreadBudget = InRenderThreadBudget;
	GameThreadBudget = InGameThreadBudget;
}

FString FPerfStatsRecord::GetReportString() const
{
	return FString::Printf(TEXT("%s,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f"),
		*Name,
		Record.FrameTimeTracker.GetMinValue() - Baseline.FrameTimeTracker.GetMinValue(),
		Record.FrameTimeTracker.GetAvgValue() - Baseline.FrameTimeTracker.GetAvgValue(),
		Record.FrameTimeTracker.GetMaxValue() - Baseline.FrameTimeTracker.GetMaxValue(),
		Record.RenderThreadTimeTracker.GetMinValue() - Baseline.RenderThreadTimeTracker.GetMinValue(),
		Record.RenderThreadTimeTracker.GetAvgValue() - Baseline.RenderThreadTimeTracker.GetAvgValue(),
		Record.RenderThreadTimeTracker.GetMaxValue() - Baseline.RenderThreadTimeTracker.GetMaxValue(),
		Record.GameThreadTimeTracker.GetMinValue() - Baseline.GameThreadTimeTracker.GetMinValue(),
		Record.GameThreadTimeTracker.GetAvgValue() - Baseline.GameThreadTimeTracker.GetAvgValue(),
		Record.GameThreadTimeTracker.GetMaxValue() - Baseline.GameThreadTimeTracker.GetMaxValue(),
		Record.GPUTimeTracker.GetMinValue() - Baseline.GPUTimeTracker.GetMinValue(),
		Record.GPUTimeTracker.GetAvgValue() - Baseline.GPUTimeTracker.GetAvgValue(),
		Record.GPUTimeTracker.GetMaxValue() - Baseline.GPUTimeTracker.GetMaxValue());
}

FString FPerfStatsRecord::GetBaselineString() const
{
	return FString::Printf(TEXT("%s,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f"),
		*Name,
		Baseline.FrameTimeTracker.GetMinValue(),
		Baseline.FrameTimeTracker.GetAvgValue(),
		Baseline.FrameTimeTracker.GetMaxValue(),
		Baseline.RenderThreadTimeTracker.GetMinValue(),
		Baseline.RenderThreadTimeTracker.GetAvgValue(),
		Baseline.RenderThreadTimeTracker.GetMaxValue(),
		Baseline.GameThreadTimeTracker.GetMinValue(),
		Baseline.GameThreadTimeTracker.GetAvgValue(),
		Baseline.GameThreadTimeTracker.GetMaxValue(),
		Baseline.GPUTimeTracker.GetMinValue(),
		Baseline.GPUTimeTracker.GetAvgValue(),
		Baseline.GPUTimeTracker.GetMaxValue());
}

FString FPerfStatsRecord::GetRecordString() const
{
	return FString::Printf(TEXT("%s,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f"),
		*Name,
		Record.FrameTimeTracker.GetMinValue(),
		Record.FrameTimeTracker.GetAvgValue(),
		Record.FrameTimeTracker.GetMaxValue(),
		Record.RenderThreadTimeTracker.GetMinValue(),
		Record.RenderThreadTimeTracker.GetAvgValue(),
		Record.RenderThreadTimeTracker.GetMaxValue(),
		Record.GameThreadTimeTracker.GetMinValue(),
		Record.GameThreadTimeTracker.GetAvgValue(),
		Record.GameThreadTimeTracker.GetMaxValue(),
		Record.GPUTimeTracker.GetMinValue(),
		Record.GPUTimeTracker.GetAvgValue(),
		Record.GPUTimeTracker.GetMaxValue());
}

FString FPerfStatsRecord::GetOverBudgetString() const
{
	double Min, Max, Avg;
	GetRenderThreadTimes(Min, Max, Avg);
	float RTMax = (float)Max;
	float RTBudgetFrac = (float)(Max / RenderThreadBudget);
	GetGameThreadTimes(Min, Max, Avg);
	float GTMax = (float)Max;
	float GTBudgetFrac = (float)(Max / GameThreadBudget);
	GetGPUTimes(Min, Max, Avg);
	float GPUMax = (float)Max;
	float GPUBudgetFrac = (float)(Max / GPUBudget);

	return FString::Printf(TEXT("%s,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f"),
		*Name,
		RTMax,
		RenderThreadBudget,
		RTBudgetFrac,
		GTMax,
		GameThreadBudget,
		GTBudgetFrac,
		GPUMax,
		GPUBudget,
		GPUBudgetFrac
		);
}

bool FPerfStatsRecord::IsWithinGPUBudget()const
{
	double Min, Max, Avg;
	GetGPUTimes(Min, Max, Avg);
	return Max <= GPUBudget;
}

bool FPerfStatsRecord::IsWithinGameThreadBudget()const
{
	double Min, Max, Avg;
	GetGameThreadTimes(Min, Max, Avg);
	return Max <= GameThreadBudget;
}

bool FPerfStatsRecord::IsWithinRenderThreadBudget()const
{
	double Min, Max, Avg;
	GetRenderThreadTimes(Min, Max, Avg);
	return Max <= RenderThreadBudget;
}

void FPerfStatsRecord::GetGPUTimes(double& OutMin, double& OutMax, double& OutAvg)const
{
	OutMin = Record.GPUTimeTracker.GetMinValue() - Baseline.GPUTimeTracker.GetMinValue();
	OutMax = Record.GPUTimeTracker.GetMaxValue() - Baseline.GPUTimeTracker.GetMaxValue();
	OutAvg = Record.GPUTimeTracker.GetAvgValue() - Baseline.GPUTimeTracker.GetAvgValue();
}

void FPerfStatsRecord::GetGameThreadTimes(double& OutMin, double& OutMax, double& OutAvg)const
{
	OutMin = Record.GameThreadTimeTracker.GetMinValue() - Baseline.GameThreadTimeTracker.GetMinValue();
	OutMax = Record.GameThreadTimeTracker.GetMaxValue() - Baseline.GameThreadTimeTracker.GetMaxValue();
	OutAvg = Record.GameThreadTimeTracker.GetAvgValue() - Baseline.GameThreadTimeTracker.GetAvgValue();
}

void FPerfStatsRecord::GetRenderThreadTimes(double& OutMin, double& OutMax, double& OutAvg)const
{
	OutMin = Record.RenderThreadTimeTracker.GetMinValue() - Baseline.RenderThreadTimeTracker.GetMinValue();
	OutMax = Record.RenderThreadTimeTracker.GetMaxValue() - Baseline.RenderThreadTimeTracker.GetMaxValue();
	OutAvg = Record.RenderThreadTimeTracker.GetAvgValue() - Baseline.RenderThreadTimeTracker.GetAvgValue();
}

void FPerfStatsRecord::Sample(UWorld* World, float DeltaSeconds, bool bBaseline)
{
	check(World);

	const FStatUnitData* StatUnitData = World->GetGameViewport()->GetStatUnitData();
	check(StatUnitData);

	if (bBaseline)
	{
		Baseline.FrameTimeTracker.AddSample(StatUnitData->RawFrameTime);
		Baseline.GameThreadTimeTracker.AddSample(FPlatformTime::ToMilliseconds(GGameThreadTime));
		Baseline.RenderThreadTimeTracker.AddSample(FPlatformTime::ToMilliseconds(GRenderThreadTime));
		Baseline.GPUTimeTracker.AddSample(FPlatformTime::ToMilliseconds(RHIGetGPUFrameCycles()));
		Baseline.NumFrames++;
		Baseline.SumTimeSeconds += DeltaSeconds;
	}
	else
	{
		Record.FrameTimeTracker.AddSample(StatUnitData->RawFrameTime);
		Record.GameThreadTimeTracker.AddSample(FPlatformTime::ToMilliseconds(GGameThreadTime));
		Record.RenderThreadTimeTracker.AddSample(FPlatformTime::ToMilliseconds(GRenderThreadTime));
		Record.GPUTimeTracker.AddSample(FPlatformTime::ToMilliseconds(RHIGetGPUFrameCycles()));
		Record.NumFrames++;
		Record.SumTimeSeconds += DeltaSeconds;
	}
}

UAutomationPerformaceHelper::UAutomationPerformaceHelper()
: bRecordingBasicStats(false)
, bRecordingBaselineBasicStats(false)
, bRecordingCPUCapture(false)
, bRecordingStatsFile(false)
, bGPUTraceIfBelowBudget(false)
{
}

UWorld* UAutomationPerformaceHelper::GetWorld() const
{
	UWorld* OuterWorld = GetOuter()->GetWorld();
	ensureAsRuntimeWarning(OuterWorld != nullptr);
	return OuterWorld;
}

void UAutomationPerformaceHelper::BeginRecordingBaseline(FString RecordName)
{
	if (UWorld* World = GetWorld())
	{
		bRecordingBasicStats = true;
		bRecordingBaselineBasicStats = true;
		bGPUTraceIfBelowBudget = false;
		Records.Add(FPerfStatsRecord(RecordName));
		GEngine->SetEngineStat(World, World->GetGameViewport(), TEXT("Unit"), true);
	}
}

void UAutomationPerformaceHelper::EndRecordingBaseline()
{
	bRecordingBaselineBasicStats = false;
	bRecordingBasicStats = false;
}

void UAutomationPerformaceHelper::BeginRecording(FString RecordName, float InGPUBudget, float InRenderThreadBudget, float InGameThreadBudget)
{
	if (UWorld* World = GetWorld())
	{
		//Ensure we're recording engine stats.
		GEngine->SetEngineStat(World, World->GetGameViewport(), TEXT("Unit"), true);
		bRecordingBasicStats = true;
		bRecordingBaselineBasicStats = false;
		bGPUTraceIfBelowBudget = false;

		FPerfStatsRecord* CurrRecord = GetCurrentRecord();
		if (!CurrRecord || CurrRecord->Name != RecordName)
		{
			Records.Add(FPerfStatsRecord(RecordName));
			CurrRecord = GetCurrentRecord();
		}

		check(CurrRecord);
		CurrRecord->SetBudgets(InGPUBudget, InRenderThreadBudget, InGameThreadBudget);
	}
}

void UAutomationPerformaceHelper::EndRecording()
{
	if (const FPerfStatsRecord* Record = GetCurrentRecord())
	{
		UE_LOGF(LogFunctionalTest, Log, "Finished Perf Stats Record:\n%ls", *Record->GetReportString());
	}
	bRecordingBasicStats = false;
}

void UAutomationPerformaceHelper::Tick(float DeltaSeconds)
{
	if (bRecordingBasicStats)
	{
		Sample(DeltaSeconds);
	}

	if (bGPUTraceIfBelowBudget)
	{
		if (!IsCurrentRecordWithinGPUBudget())
		{
			FString PathName = FPaths::ProfilingDir();			
			GGPUTraceFileName = PathName / CreateProfileFilename(GetCurrentRecord()->Name, TEXT(".rtt"), true);
			UE_LOGF(LogFunctionalTest, Log, "Functional Test has fallen below GPU budget. Performing GPU trace.");

			GEngine->AddOnScreenDebugMessage(-1, 5.f, FColor::Red, TEXT("Performed GPU Thred Trace!"));

			//Only perform one trace per test. 
			bGPUTraceIfBelowBudget = false;
		}
	}

	//Other stats need ticking?
}

void UAutomationPerformaceHelper::Sample(float DeltaSeconds)
{
	if (UWorld* World = GetWorld())
	{
		int32 Index = Records.Num() - 1;
		if (Index >= 0 && bRecordingBasicStats)
		{
			Records[Index].Sample(World, DeltaSeconds, bRecordingBaselineBasicStats);
		}
	}
}

void UAutomationPerformaceHelper::WriteLogFile(const FString& CaptureDir, const FString& CaptureExtension)
{
	FString PathName = FPaths::ProfilingDir();
	if (!CaptureDir.IsEmpty())
	{
		PathName = PathName + (CaptureDir + TEXT("/"));
		IFileManager::Get().MakeDirectory(*PathName);
	}

	FString Extension = CaptureExtension;
	if (Extension.IsEmpty())
	{
		Extension = TEXT("perf.csv");
	}

	const FString Filename = CreateProfileFilename(CaptureExtension, true);
	const FString FilenameFull = PathName + Filename;
	
	const FString OverBudgetTableHeader = TEXT("TestName, MaxRT, RT Budget, RT Frac, MaxGT, GT Budget, GT Frac, MaxGPU, GPU Budget, GPU Frac\n");
	FString OverbudgetTable;
	const FString DataTableHeader = TEXT("TestName,MinFrameTime,AvgFrameTime,MaxFrameTime,MinRT,AvgRT,MaxRT,MinGT,AvgGT,MaxGT,MinGPU,AvgGPU,MaxGPU\n");
	FString AdjustedTable;
	FString RecordTable;
	FString BaselineTable;
	for (FPerfStatsRecord& Record : Records)
	{
		AdjustedTable += Record.GetReportString() + FString(TEXT("\n"));
		RecordTable += Record.GetRecordString() + FString(TEXT("\n"));
		BaselineTable += Record.GetBaselineString() + FString(TEXT("\n"));

		if (!Record.IsWithinGPUBudget() || !Record.IsWithinRenderThreadBudget() || !Record.IsWithinGameThreadBudget())
		{
			OverbudgetTable += Record.GetOverBudgetString() + FString(TEXT("\n"));
		}
	}

	FString FileContents = FString::Printf(TEXT("Over Budget Tests\n%s%s\nAdjusted Results\n%s%s\nRaw Results\n%s%s\nBaseline Results\n%s%s\n"), 
		*OverBudgetTableHeader, *OverbudgetTable, *DataTableHeader, *AdjustedTable, *DataTableHeader, *RecordTable, *DataTableHeader, *BaselineTable);

	FFileHelper::SaveStringToFile(FileContents, *FilenameFull);

	UE_LOGF(LogTemp, Display, "Finished test, wrote file to %ls", *FilenameFull);

	Records.Empty();
	bRecordingBasicStats = false;
	bRecordingBaselineBasicStats = false;
}

bool UAutomationPerformaceHelper::IsRecording()const 
{
	return bRecordingBasicStats;
}

void UAutomationPerformaceHelper::OnBeginTests()
{
	OutputFileBase = CreateProfileFilename(TEXT(""), true);
	StartOfTestingTime = FDateTime::Now().ToString();
}

void UAutomationPerformaceHelper::OnAllTestsComplete()
{
	if (bRecordingBaselineBasicStats)
	{
		EndRecordingBaseline();
	}

	if (bRecordingBasicStats)
	{
		EndRecording();
	}

	if (bRecordingCPUCapture)
	{
		StopCPUProfiling();
	}

	if (bRecordingStatsFile)
	{
		EndStatsFile();
	}
	
	bGPUTraceIfBelowBudget = false;

	if (Records.Num() > 0)
	{
		WriteLogFile(TEXT(""), TEXT("perf.csv"));
	}
}

bool UAutomationPerformaceHelper::IsCurrentRecordWithinGPUBudget()const
{
	if (const FPerfStatsRecord* Curr = GetCurrentRecord())
	{
		return Curr->IsWithinGPUBudget();
	}
	return true;
}

bool UAutomationPerformaceHelper::IsCurrentRecordWithinGameThreadBudget()const
{
	if (const FPerfStatsRecord* Curr = GetCurrentRecord())
	{
		return Curr->IsWithinGameThreadBudget();
	}
	return true;
}

bool UAutomationPerformaceHelper::IsCurrentRecordWithinRenderThreadBudget()const
{
	if (const FPerfStatsRecord* Curr = GetCurrentRecord())
	{
		return Curr->IsWithinRenderThreadBudget();
	}
	return true;
}

const FPerfStatsRecord* UAutomationPerformaceHelper::GetCurrentRecord()const
{
	int32 Index = Records.Num() - 1;
	if (Index >= 0)
	{
		return &Records[Index];
	}
	return nullptr;
}
FPerfStatsRecord* UAutomationPerformaceHelper::GetCurrentRecord()
{
	int32 Index = Records.Num() - 1;
	if (Index >= 0)
	{
		return &Records[Index];
	}
	return nullptr;
}

void UAutomationPerformaceHelper::StartCPUProfiling()
{
#if UE_EXTERNAL_PROFILING_ENABLED
	UE_LOGF(LogFunctionalTest, Log, "START PROFILING...");
	ExternalProfiler.StartProfiler(false);
#endif
}

void UAutomationPerformaceHelper::StopCPUProfiling()
{
#if UE_EXTERNAL_PROFILING_ENABLED
	UE_LOGF(LogFunctionalTest, Log, "STOP PROFILING...");
	ExternalProfiler.StopProfiler();
#endif
}

void UAutomationPerformaceHelper::TriggerGPUTraceIfRecordFallsBelowBudget()
{
	bGPUTraceIfBelowBudget = true;
}

void UAutomationPerformaceHelper::BeginStatsFile(const FString& RecordName)
{
	if (UWorld* World = GetWorld())
	{
		FString MapName = World->GetMapName();
		FString Cmd = FString::Printf(TEXT("Stat StartFile %s-%s/%s.uestats"), *MapName, *StartOfTestingTime, *RecordName);
		GEngine->Exec(World, *Cmd);
	}
}

void UAutomationPerformaceHelper::EndStatsFile()
{
	if (UWorld* World = GetWorld())
	{
		GEngine->Exec(World, TEXT("Stat StopFile"));
	}
}
