// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalTestBase.h"
#include "FunctionalTestingModule.h"
#include "AutomationControllerSettings.h"

#if ENABLE_RHI_VALIDATION
#include "RHIValidationCommon.h"
#endif

// statics
bool FFunctionalTestBase::bIsFunctionalTestRunning;
FString FFunctionalTestBase::ActiveTestName;


FFunctionalTestBase::FFunctionalTestBase(const FString& InName, const bool bInComplexTask)
	: FAutomationTestBase(InName, bInComplexTask)
{
	bSuppressLogs = false;
	bIsFunctionalTestRunning = false;
	// CDO not available at this point
	bSuppressLogErrors = false;
	bSuppressLogWarnings = false;
	bElevateLogWarningsToErrors = true;
}

bool FFunctionalTestBase::SuppressLogs()
{
#if ENABLE_RHI_VALIDATION
	if (GRHIValidationEnabled && !bSuppressLogs)
	{
		// While RHI Validation is enabled, do not suppress log unless explicitly enabled
		return false;
	}
#endif
	return bSuppressLogs || !IsFunctionalTestRunning();
}

bool FFunctionalTestBase::ShouldCaptureLogCategory(const class FName& Category) const
{
#if ENABLE_RHI_VALIDATION
	if (GRHIValidationEnabled && !IsFunctionalTestRunning())
	{
		// If capturing log while functional test is not running, only filter in LogRHI channel. 
		return Category == FName(TEXT("LogRHI"));
	}
#endif
	return true;
}

void FFunctionalTestBase::SetLogErrorAndWarningHandlingToDefault()
{
	// Set to project defaults
	UAutomationControllerSettings* Settings = UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();
	bSuppressLogErrors = Settings->bSuppressLogErrors;
	bSuppressLogWarnings = Settings->bSuppressLogWarnings;
	bElevateLogWarningsToErrors = Settings->bElevateLogWarningsToErrors;
}

/**
	 * Marks us as actively running a functional test
	 */
void FFunctionalTestBase::SetFunctionalTestRunning(const FString& InName)
{
	if (bIsFunctionalTestRunning)
	{
		UE_LOGF(LogFunctionalTest, Error, "A Functional test is already running!");
	}

	if (InName.Len() == 0)
	{
		UE_LOGF(LogFunctionalTest, Error, "No test name supplied!");
	}

	bIsFunctionalTestRunning = true;
	ActiveTestName = InName;
}

/**
 * Marks us as no longer running a test
 */
void FFunctionalTestBase::SetFunctionalTestComplete(const FString& InName)
{
	if (!bIsFunctionalTestRunning)
	{
		UE_LOGF(LogFunctionalTest, Error, "SetFunctionalComplete() called with no active test");
	}

	if (InName != ActiveTestName)
	{
		UE_LOGF(LogFunctionalTest, Error, "SetFunctionalComplete: Complete name %ls did not match active name %ls", *InName, *ActiveTestName);
	}

	bIsFunctionalTestRunning = false;
	ActiveTestName = TEXT("");
}
