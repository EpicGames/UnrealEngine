// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalTestGameMode.h"
#include "GameFramework/SpectatorPawn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FunctionalTestGameMode)

AFunctionalTestGameMode::AFunctionalTestGameMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DefaultPawnClass = ASpectatorPawn::StaticClass();
}

