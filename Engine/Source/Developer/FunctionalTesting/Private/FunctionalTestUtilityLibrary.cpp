// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalTestUtilityLibrary.h"
#include "TraceQueryTestResults.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FunctionalTestUtilityLibrary)

UTraceQueryTestResults* UFunctionalTestUtilityLibrary::TraceChannelTestUtil(UObject* WorldContextObject, const FTraceChannelTestBatchOptions& BatchOptions, const FVector Start, const FVector End, float SphereCapsuleRadius, float CapsuleHalfHeight, FVector BoxHalfSize, const FRotator Orientation, ETraceTypeQuery TraceChannel, TArray<TEnumAsByte<EObjectTypeQuery> > ObjectTypes, FName ProfileName, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, bool bIgnoreSelf, EDrawDebugTrace::Type DrawDebugType, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime)
{
	UTraceQueryTestResults* Results = NewObject<UTraceQueryTestResults>();
	if(Results)
	{
		//line
		Results->BatchOptions = BatchOptions;
		if(BatchOptions.bLineTrace)
		{
			if(BatchOptions.bChannelTrace)
			{
				Results->ChannelResults.LineResults.bSingleResult = UKismetSystemLibrary::LineTraceSingle(WorldContextObject, Start, End, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ChannelResults.LineResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ChannelResults.LineResults.bMultiResult = UKismetSystemLibrary::LineTraceMulti(WorldContextObject, Start, End, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ChannelResults.LineResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}
			
			if(BatchOptions.bObjectsTrace)
			{
				Results->ObjectResults.LineResults.bSingleResult = UKismetSystemLibrary::LineTraceSingleForObjects(WorldContextObject, Start, End, ObjectTypes, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ObjectResults.LineResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ObjectResults.LineResults.bMultiResult = UKismetSystemLibrary::LineTraceMultiForObjects(WorldContextObject, Start, End, ObjectTypes, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ObjectResults.LineResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}
			
			if(BatchOptions.bProfileTrace)
			{
				Results->ProfileResults.LineResults.bSingleResult = UKismetSystemLibrary::LineTraceSingleByProfile(WorldContextObject, Start, End, ProfileName, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ProfileResults.LineResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ProfileResults.LineResults.bMultiResult = UKismetSystemLibrary::LineTraceMultiByProfile(WorldContextObject, Start, End, ProfileName, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ProfileResults.LineResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}
		}
		

		//sphere
		if(BatchOptions.bSphereTrace)
		{
			if (BatchOptions.bChannelTrace)
			{
				Results->ChannelResults.SphereResults.bSingleResult = UKismetSystemLibrary::SphereTraceSingle(WorldContextObject, Start, End, SphereCapsuleRadius, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ChannelResults.SphereResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ChannelResults.SphereResults.bMultiResult = UKismetSystemLibrary::SphereTraceMulti(WorldContextObject, Start, End, SphereCapsuleRadius, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ChannelResults.SphereResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}

			if (BatchOptions.bObjectsTrace)
			{
				Results->ObjectResults.SphereResults.bSingleResult = UKismetSystemLibrary::SphereTraceSingleForObjects(WorldContextObject, Start, End, SphereCapsuleRadius, ObjectTypes, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ObjectResults.SphereResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ObjectResults.SphereResults.bMultiResult = UKismetSystemLibrary::SphereTraceMultiForObjects(WorldContextObject, Start, End, SphereCapsuleRadius, ObjectTypes, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ObjectResults.SphereResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}

			if (BatchOptions.bProfileTrace)
			{
				Results->ProfileResults.SphereResults.bSingleResult = UKismetSystemLibrary::SphereTraceSingleByProfile(WorldContextObject, Start, End, SphereCapsuleRadius, ProfileName, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ProfileResults.SphereResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ProfileResults.SphereResults.bMultiResult = UKismetSystemLibrary::SphereTraceMultiByProfile(WorldContextObject, Start, End, SphereCapsuleRadius, ProfileName, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ProfileResults.SphereResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}
		}
		
		//capsule
		if(BatchOptions.bCapsuleTrace)
		{
			if (BatchOptions.bChannelTrace)
			{
				Results->ChannelResults.CapsuleResults.bSingleResult = UKismetSystemLibrary::CapsuleTraceSingle(WorldContextObject, Start, End, SphereCapsuleRadius, CapsuleHalfHeight, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ChannelResults.CapsuleResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ChannelResults.CapsuleResults.bMultiResult = UKismetSystemLibrary::CapsuleTraceMulti(WorldContextObject, Start, End, SphereCapsuleRadius, CapsuleHalfHeight, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ChannelResults.CapsuleResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}
			
			if (BatchOptions.bObjectsTrace)
			{
				Results->ObjectResults.CapsuleResults.bSingleResult = UKismetSystemLibrary::CapsuleTraceSingleForObjects(WorldContextObject, Start, End, SphereCapsuleRadius, CapsuleHalfHeight, ObjectTypes, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ObjectResults.CapsuleResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ObjectResults.CapsuleResults.bMultiResult = UKismetSystemLibrary::CapsuleTraceMultiForObjects(WorldContextObject, Start, End, SphereCapsuleRadius, CapsuleHalfHeight, ObjectTypes, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ObjectResults.CapsuleResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}

			if (BatchOptions.bProfileTrace)
			{
				Results->ProfileResults.CapsuleResults.bSingleResult = UKismetSystemLibrary::CapsuleTraceSingleByProfile(WorldContextObject, Start, End, SphereCapsuleRadius, CapsuleHalfHeight, ProfileName, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ProfileResults.CapsuleResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ProfileResults.CapsuleResults.bMultiResult = UKismetSystemLibrary::CapsuleTraceMultiByProfile(WorldContextObject, Start, End, SphereCapsuleRadius, CapsuleHalfHeight, ProfileName, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ProfileResults.CapsuleResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}
		}

		//box
		if(BatchOptions.bBoxTrace)
		{
			if (BatchOptions.bChannelTrace)
			{
				Results->ChannelResults.BoxResults.bSingleResult = UKismetSystemLibrary::BoxTraceSingle(WorldContextObject, Start, End, BoxHalfSize, Orientation, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ChannelResults.BoxResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ChannelResults.BoxResults.bMultiResult = UKismetSystemLibrary::BoxTraceMulti(WorldContextObject, Start, End, BoxHalfSize, Orientation, TraceChannel, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ChannelResults.BoxResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}
			
			if (BatchOptions.bObjectsTrace)
			{
				Results->ObjectResults.BoxResults.bSingleResult = UKismetSystemLibrary::BoxTraceSingleForObjects(WorldContextObject, Start, End, BoxHalfSize, Orientation, ObjectTypes, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ObjectResults.BoxResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ObjectResults.BoxResults.bMultiResult = UKismetSystemLibrary::BoxTraceMultiForObjects(WorldContextObject, Start, End, BoxHalfSize, Orientation, ObjectTypes, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ObjectResults.BoxResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}

			if (BatchOptions.bProfileTrace)
			{
				Results->ProfileResults.BoxResults.bSingleResult = UKismetSystemLibrary::BoxTraceSingleByProfile(WorldContextObject, Start, End, BoxHalfSize, Orientation, ProfileName, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ProfileResults.BoxResults.SingleHit, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
				Results->ProfileResults.BoxResults.bMultiResult = UKismetSystemLibrary::BoxTraceMultiByProfile(WorldContextObject, Start, End, BoxHalfSize, Orientation, ProfileName, bTraceComplex, ActorsToIgnore, DrawDebugType, Results->ProfileResults.BoxResults.MultiHits, bIgnoreSelf, TraceColor, TraceHitColor, DrawTime);
			}
		}

		Results->CaptureNames();
	}

	return Results;
}
