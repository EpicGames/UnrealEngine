// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalTestingHelper.h"
#include "FunctionalTestingModule.h"

bool FWaitForFTestsToFinish::Update()
{
	return IFunctionalTestingModule::Get().IsRunning() == false;
}

bool FTriggerFTests::Update()
{
	IFunctionalTestingModule& Module = IFunctionalTestingModule::Get();
	if (Module.IsFinished())
	{
		// if tests have been already triggered by level script just make sure it's not looping
		if (Module.IsRunning())
		{
			Module.SetLooping(false);
		}
		else
		{
			Module.RunAllTestsOnMap(false, false);
		}
	}

	return true;
}

bool FTriggerFTest::Update()
{
	IFunctionalTestingModule& Module = IFunctionalTestingModule::Get();
	if (Module.IsFinished())
	{
		// if tests have been already triggered by level script just make sure it's not looping
		if (Module.IsRunning())
		{
			IFunctionalTestingModule::Get().SetLooping(false);
		}
		else
		{
			IFunctionalTestingModule::Get().RunTestOnMap(TestName, false, false);
		}
	}

	return true;
}

bool FStartFTestsOnMap::Update()
{
	// This should now be handled by your IsReady override of the functional test.
	//should really be wait until the map is properly loaded....in PIE or gameplay....
	//ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(15.f));

	ADD_LATENT_AUTOMATION_COMMAND(FTriggerFTests);
	ADD_LATENT_AUTOMATION_COMMAND(FWaitForFTestsToFinish);

	return true;
}

bool FStartFTestOnMap::Update()
{
	ADD_LATENT_AUTOMATION_COMMAND(FTriggerFTest(TestName));
	ADD_LATENT_AUTOMATION_COMMAND(FWaitForFTestsToFinish);

	return true;
}