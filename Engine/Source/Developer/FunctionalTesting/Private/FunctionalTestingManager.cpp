// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalTestingManager.h"
#include "TimerManager.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "FunctionalTestingModule.h"
#include "NavigationSystem.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Misc/RuntimeErrors.h"
#include "GameFramework/PlayerController.h"
#if WITH_EDITOR
#include "TickableEditorObject.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(FunctionalTestingManager)

namespace FFunctionalTesting
{
	const TCHAR* ReproStringTestSeparator = TEXT("@");
	const TCHAR* ReproStringParamsSeparator = TEXT("#");
}

#if WITH_EDITOR
namespace UE::Private
{

class FTickEditorTest final : public FTickableEditorObject
{
public:
	FTickEditorTest(TObjectPtr<AFunctionalTest> InActor)
		: FTickableEditorObject()
		, Actor(InActor)
	{
	}

private:
	virtual void Tick( float DeltaTime ) override;
	virtual ETickableTickType GetTickableTickType() const override 
	{ 
		return ETickableTickType::Always; 
	}
	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FTickEditorTest, STATGROUP_Tickables);
	}

	TStrongObjectPtr<AFunctionalTest> Actor;
};

void FTickEditorTest::Tick(float DeltaTime)
{
	const TGuardValue ScriptExecutionGuard(GAllowActorScriptExecutionInEditor, true);
	if(Actor)
	{
		Actor->Tick(DeltaTime);
	}
}

}
#endif

UFunctionalTestingManager::UFunctionalTestingManager( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, bIsRunning(false)
	, bFinished(false)
	, bLooped(false)
	, bInitialDelayApplied(false)
    , bIsTearingDown(false)
	, CurrentIteration(INDEX_NONE)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		TestFinishedObserver = FFunctionalTestDoneSignature::CreateUObject(this, &UFunctionalTestingManager::OnTestDone);
	}
}

void UFunctionalTestingManager::SetUpTests()
{
	OnSetupTests.Broadcast();
}

struct FSortTestActorsByName
{
	FORCEINLINE bool operator()(const AFunctionalTest& A, const AFunctionalTest& B) const
	{
		return A.GetName() > B.GetName(); 
	}
};

bool UFunctionalTestingManager::RunAllFunctionalTests(UObject* WorldContextObject, bool bNewLog /*= true*/, bool bRunLooped /*= true*/, FString ReproString /*= TEXT("")*/)
{
	UFunctionalTestingManager* Manager = GetManager(WorldContextObject);
	if (!ensureAsRuntimeWarning(Manager != nullptr))
	{
		return false;
	}

	if (Manager->bIsRunning)
	{
		UE_LOGF(LogFunctionalTest, Log, "Functional tests are already running.");
		return true;
	}
	
	UWorld* World = GEngine->GetWorldFromContextObjectChecked(WorldContextObject);
	GEngine->ForceGarbageCollection(true);

	Manager->bFinished = false;
	Manager->bLooped = bRunLooped;
	Manager->CurrentIteration = 0;
	Manager->TestsLeft.Reset();
	Manager->AllTests.Reset();
	Manager->SetReproString(ReproString);

	Manager->SetUpTests();

	if (Manager->TestReproStrings.Num() > 0)
	{
		UE_LOGF(LogFunctionalTest, Log, "Running tests indicated by Repro String: %ls", *ReproString);
		Manager->TriggerFirstValidTest();
	}
	else
	{
		for (TActorIterator<APhasedAutomationActorBase> It(World); It; ++It)
		{
			APhasedAutomationActorBase* PAA = (*It);
			Manager->OnTestsComplete.AddDynamic(PAA, &APhasedAutomationActorBase::OnFunctionalTestingComplete); 
			Manager->OnTestsBegin.AddDynamic(PAA, &APhasedAutomationActorBase::OnFunctionalTestingBegin); 
		}

		for (TActorIterator<AFunctionalTest> It(World); It; ++It)
		{
			AFunctionalTest* Test = (*It);
			if (Test != nullptr && Test->IsEnabled() == true)
			{
				Manager->AllTests.Add(Test);
			}
		}

		Manager->AllTests.Sort(FSortTestActorsByName());

		if (Manager->AllTests.Num() > 0)
		{
			Manager->TestsLeft = Manager->AllTests;
			
			Manager->OnTestsBegin.Broadcast();

			Manager->TriggerFirstValidTest();
		}
	}

	if (Manager->bIsRunning == false)
	{
		UE_LOGF(LogFunctionalTest, Warning, "No tests defined on map or . DONE.");
		return false;
	}

	return true;
}

void UFunctionalTestingManager::TriggerFirstValidTest()
{
	UWorld* World = GetWorld();
	check(World);
	bIsRunning = true;

	bool bIsWorldInitialized =
		(World->AreActorsInitialized() &&
		(!World->GetWorldSettings()->IsNavigationSystemEnabled() || !UNavigationSystemV1::IsNavigationBeingBuilt(World)) &&
		(World->GetNumPlayerControllers() != 0) && World->GetFirstPlayerController()->GetPawnOrSpectator() != nullptr);
#if WITH_EDITOR
	const bool bInEditorWorld = (World->WorldType == EWorldType::Editor);
	// editor world doesn't need to wait for navigation, controllers, pawns, or actor initialization:
	bIsWorldInitialized |= bInEditorWorld;
	bInitialDelayApplied |= bInEditorWorld; // skip initial delay in editor world, wastes time and would require a different timer manager
	// The ability to run actor based tests in the editor world is new
	// so I'm adding a logging statement so that we have a fingerprint
	// in the log of any behavior change:
	if(bInEditorWorld)
	{
		UE_LOGF(LogTemp, Log, "Running %ls in Editor World", TestReproStrings.Num() > 0 ? *TestReproStrings[0] : TEXT("unknown test"));
	}
#endif
	
	if (bInitialDelayApplied == true && bIsWorldInitialized)
	{
		bIsRunning = RunFirstValidTest();
		if (bIsRunning == false)
		{
			AllTestsDone();
		}
	}
	else
	{
		bInitialDelayApplied = true;
		static const float WaitingTime = 0.25f;
		World->GetTimerManager().SetTimer(TriggerFirstValidTestTimerHandle, this, &UFunctionalTestingManager::TriggerFirstValidTest, WaitingTime);
	}
}

UFunctionalTestingManager* UFunctionalTestingManager::GetManager(UObject* WorldContext)
{
	UFunctionalTestingManager* Manager = IFunctionalTestingModule::Get().GetCurrentManager();

	if (Manager == nullptr)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContext, EGetWorldErrorMode::LogAndReturnNull))
		{
			if (World->WorldType == EWorldType::PIE || World->WorldType == EWorldType::Game
				#if WITH_EDITOR
				|| World->WorldType == EWorldType::Editor
				#endif
				)
			{
				Manager = NewObject<UFunctionalTestingManager>(World);
				IFunctionalTestingModule::Get().SetManager(Manager);

				// add to root and get notified on world cleanup to remove from root on map cleanup
				Manager->AddToRoot();
				FWorldDelegates::OnWorldCleanup.AddUObject(Manager, &UFunctionalTestingManager::OnWorldCleanedUp);
			}
		}
		else
		{
			ensureMsgf(false, TEXT("Tried to add a functional test manager to a non-game world."));
		}
	}

	return Manager;
}

UWorld* UFunctionalTestingManager::GetWorld() const
{
	return GEngine->GetWorldFromContextObjectChecked(GetOuter());
}

void UFunctionalTestingManager::OnWorldCleanedUp(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	UWorld* MyWorld = GetWorld();
	if (MyWorld == World)
	{
		RemoveFromRoot();

		// Clear the functional test manager once the world is removed.
		IFunctionalTestingModule::Get().SetManager(nullptr);
        
        bIsTearingDown = true;
	}
}

void UFunctionalTestingManager::OnTestDone(AFunctionalTest* FTest)
{
	// add a delay
	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.Requesting to build next tile if necessary"),
		STAT_FSimpleDelegateGraphTask_RequestingToBuildNextTileIfNecessary,
		STATGROUP_TaskGraphTasks);

	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateUObject(this, &UFunctionalTestingManager::NotifyTestDone, FTest),
		GET_STATID(STAT_FSimpleDelegateGraphTask_RequestingToBuildNextTileIfNecessary), NULL, ENamedThreads::GameThread);
}

void UFunctionalTestingManager::NotifyTestDone(AFunctionalTest* FTest)
{
#if WITH_EDITOR
	EditorTick.Reset();
#endif

	if (FTest->OnWantsReRunCheck() == false && FTest->WantsToRunAgain() == false)
	{
		//We can also do named reruns. These are lower priority than those triggered above.
		//These names can be queried by phases to alter behavior in re-runs.
		if (FTest->RerunCauses.Num() > 0)
		{
			FTest->CurrentRerunCause = FTest->RerunCauses.Pop();
		}
		else
		{
			TestsLeft.RemoveSingle(FTest);
			FTest->CleanUp();
		}
	}

	if ((TestsLeft.Num() > 0 || TestReproStrings.Num() > 0) && !bIsTearingDown)
	{
		bIsRunning = RunFirstValidTest();
	}
	else
	{
		bIsRunning = false;
	}

	if (bIsRunning == false)
	{
		AllTestsDone();
	}
}

void UFunctionalTestingManager::AllTestsDone()
{
	if (bLooped == true)
	{
		++CurrentIteration;

		// reset
		ensure(TestReproStrings.Num() == 0);
		SetReproString(StartingReproString);
		TestsLeft = AllTests;

		UE_LOGF(LogFunctionalTest, Log, "----- Starting iteration %d -----", CurrentIteration);
		bIsRunning = RunFirstValidTest();
		if (bIsRunning == false)
		{
			UE_LOGF(LogFunctionalTest, Warning, "Failed to start another iteration.");
		}
	}
	else
	{
		OnTestsComplete.Broadcast();
		bFinished = true;
		#if WITH_EDITOR
		EditorTick.Reset();
		#endif
		IFunctionalTestingModule::Get().SetManager(nullptr);
		RemoveFromRoot();
	}
}

bool UFunctionalTestingManager::RunFirstValidTest()
{
	bool bTestSuccessfullyTriggered = false;

	if (TestReproStrings.Num() > 0)
	{
		UWorld* World = GetWorld();

		if (World == nullptr)
		{
			UE_LOGF(LogFunctionalTest, Warning, "Unable to find testing world!");
			return bTestSuccessfullyTriggered;
		}

		while (TestReproStrings.Num() > 0)
		{
			TArray<FString> TestParams;
			const FString SingleTestReproString = TestReproStrings[0];
			TestReproStrings.RemoveAt(0);

			SingleTestReproString.ParseIntoArray(TestParams, TEXT("#"), /*InCullEmpty=*/true);
			
			if (TestParams.Num() == 0)
			{
				UE_LOGF(LogFunctionalTest, Warning, "Unable to parse \'%ls\'", *SingleTestReproString);
				continue;
			}

			// first param is the test name. Look for it		
			const FString TestName = TestParams[0];
			TestParams.RemoveAt(0, EAllowShrinking::No);

			AFunctionalTest* TestToRun = nullptr;
			for (TActorIterator<AFunctionalTest> It(World); It; ++It)
			{
				if (It->GetName() == TestName)
				{
					TestToRun = *It;
				}
			}
			
			if (TestToRun)
			{
				// Add the test we found to the tests left to run, so that if re-runs occur we continue to process this test until
				// it has finished.
				TestsLeft.Add(TestToRun);

				TestToRun->TestFinishedObserver = TestFinishedObserver;
				if (TestToRun->RunTest(TestParams))
				{
					#if WITH_EDITOR
					if(!World->HasBegunPlay() && World->WorldType == EWorldType::Editor)
					{
						EditorTick = MakeUnique<UE::Private::FTickEditorTest>(TestToRun);
					}
					else
					{
						EditorTick.Reset();
					}
					#endif
					bTestSuccessfullyTriggered = true;
					break;
				}
				else
				{
					UE_LOGF(LogFunctionalTest, Warning, "Test \'%ls\' failed to start", *TestToRun->GetName());
				}
			}
			else
			{
				UE_LOGF(LogFunctionalTest, Warning, "Unable to find test \'%ls\' in world %ls, the available tests are...", *TestName, *World->GetFullName());

				// Find Actors by type (needs a UWorld object)
				for (TActorIterator<AFunctionalTest> It(World); It; ++It)
				{
					UE_LOGF(LogFunctionalTest, Warning, "\'%ls\'.", *It->GetName());
				}
			}
		}
	}
	
	if (bTestSuccessfullyTriggered == false)
	{
		for (int32 Index = TestsLeft.Num()-1; Index >= 0; --Index)
		{
			bool bRemove = TestsLeft[Index] == NULL;
			if (TestsLeft[Index] != NULL)
			{
				ensure(TestsLeft[Index]->IsEnabled());
				TestsLeft[Index]->TestFinishedObserver = TestFinishedObserver;
				if (TestsLeft[Index]->RunTest())
				{
					if (TestsLeft[Index]->IsRunning() == true)
					{
						bTestSuccessfullyTriggered = true;
						break;
					}
					else
					{
						// test finished instantly, remove it
						bRemove = true;
					}
				}
				else
				{
					UE_LOGF(LogFunctionalTest, Warning, "Test: %ls failed to start", *TestsLeft[Index]->GetName());
					bRemove = true;
				}
			}

			if (bRemove)
			{
				TestsLeft.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
	}

	return bTestSuccessfullyTriggered;
}

void UFunctionalTestingManager::SetReproString(FString ReproString)
{
	TestReproStrings.Reset();
	StartingReproString = ReproString;
	if (ReproString.IsEmpty() == false)
	{
		ReproString.ParseIntoArray(TestReproStrings, FFunctionalTesting::ReproStringTestSeparator, /*InCullEmpty=*/true);
	}
}

