// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalTestingModule.h"
#include "AssetRegistry/AssetData.h"
#include "FunctionalTestingManager.h"
#include "Misc/CoreMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "FunctionalTest.h"
#include "EngineGlobals.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/CommandLine.h"
#include "IAutomationControllerModule.h"
#include "UObject/AssetRegistryTagsContext.h"

#define LOCTEXT_NAMESPACE "FunctionalTesting"

DEFINE_LOG_CATEGORY(LogFunctionalTest);

class FFunctionalTestingModule : public IFunctionalTestingModule
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual void RunAllTestsOnMap(bool bClearLog, bool bRunLooped) override;
	virtual void RunTestOnMap(const FString& TestName, bool bClearLog, bool bRunLooped) override;
	virtual void MarkPendingActivation() override;
	virtual bool IsActivationPending() const override;
	virtual bool IsRunning() const override;
	virtual bool IsFinished() const override;
	virtual void SetManager(class UFunctionalTestingManager* NewManager) override;
	virtual class UFunctionalTestingManager* GetCurrentManager();
	virtual void SetLooping(const bool bLoop) override;
	virtual void GetMapTests(bool bEditorOnlyTests, TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands, TArray<FString>& OutTestMapAssets) const override;
	virtual void GetMapTests(bool bEditorOnlyTests, TArray<FFunctionalTestInfo>& OutTestInfo, TArray<FString>& OutTestMapAssets) const override;

private:
	UWorld* GetTestWorld();
	void OnGetAssetTagsForWorld(const UWorld* World, FAssetRegistryTagsContext Context);


	TWeakObjectPtr<class UFunctionalTestingManager> TestManager;
	bool bPendingActivation;
};

void FFunctionalTestingModule::StartupModule() 
{
	bPendingActivation = false;
#if WITH_EDITOR
	FWorldDelegates::GetAssetTagsWithContext.AddRaw(this, &FFunctionalTestingModule::OnGetAssetTagsForWorld);
#endif
}

void FFunctionalTestingModule::ShutdownModule() 
{
#if WITH_EDITOR
	FWorldDelegates::GetAssetTagsWithContext.RemoveAll(this);
#endif
}

void FFunctionalTestingModule::OnGetAssetTagsForWorld(const UWorld* World, FAssetRegistryTagsContext Context)
{
#if WITH_EDITOR
	if (!World || World->HasAllFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TArray<FString> TestNamesRuntime;
	TArray<FString> TestNamesEditor;
	for (TActorIterator<AFunctionalTest> ActorItr(const_cast<UWorld*>(World), AFunctionalTest::StaticClass(), EActorIteratorFlags::AllActors); ActorItr; ++ActorItr)
	{
		AFunctionalTest* FunctionalTest = *ActorItr;

		if (!FunctionalTest->IsPackageExternal())
		{
			// Only include enabled tests in the list of functional tests to run.
			if (FunctionalTest->IsEnabledInWorld(World))
			{
				TArray<FString>& TestNames = IsEditorOnlyObject(FunctionalTest) ? TestNamesEditor : TestNamesRuntime;
				TestNames.Add(FString::Printf(TEXT("%s|%s|%s;"), *FunctionalTest->GetActorLabel(), *FunctionalTest->GetName(), *FunctionalTest->TestTags));
			}
		}
	}

	auto AddTestNames = [&Context](const TCHAR* TagName, TArray<FString>& TestNames)
	{
		if (!TestNames.IsEmpty())
		{
			TestNames.Sort();
			FString TestNamesStr = FString::Join(TestNames, TEXT(""));
			Context.AddTag(UObject::FAssetRegistryTag(TagName, MoveTemp(TestNamesStr), UObject::FAssetRegistryTag::TT_Hidden));
		}
	};
	AddTestNames(TEXT("TestNames"), TestNamesRuntime);
	AddTestNames(TEXT("TestNamesEditor"), TestNamesEditor);
#endif
}

void FFunctionalTestingModule::GetMapTests(bool bEditorOnlyTests, TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands, TArray<FString>& OutTestMapAssets) const
{
	TArray<FFunctionalTestInfo> TestInfo;
	GetMapTests(bEditorOnlyTests, TestInfo, OutTestMapAssets);
	for (FFunctionalTestInfo Info : TestInfo)
	{
		OutBeautifiedNames.Add(Info.BeautifiedName);
		OutTestCommands.Add(Info.TestCommand);
	}
}

void FFunctionalTestingModule::GetMapTests(bool bEditorOnlyTests, TArray<FFunctionalTestInfo>& OutTestInfo, TArray<FString>& OutTestMapAssets) const
{
	IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	if (!AssetRegistry.IsLoadingAssets())
	{
#if WITH_EDITOR
		static bool bDidScan = false;

		if (!GIsEditor && !bDidScan)
		{
			// For editor build -game, we need to do a full scan
			AssetRegistry.SearchAllAssets(true);
			bDidScan = true;
		}
#endif

		TArray<FAssetData> MapList;
		FARFilter Filter;
		Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
		Filter.bRecursiveClasses = true;
		Filter.bIncludeOnlyOnDiskAssets = true;
		if (AssetRegistry.GetAssets(Filter, /*out*/ MapList))
		{
			IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
			IAutomationControllerManagerPtr AutomationController = AutomationControllerModule.GetAutomationController();
			bool IsDeveloperDirectoryIncluded = AutomationController->IsDeveloperDirectoryIncluded();

			for (const FAssetData& MapAsset : MapList)
			{
				FString MapAssetPath = MapAsset.GetObjectPathString();
				FString MapPackageName = MapAsset.PackageName.ToString();
				if (!IsDeveloperDirectoryIncluded && MapPackageName.Find(TEXT("/Game/Developers")) == 0) continue;
				FString PartialSuiteName = MapPackageToAutomationPath(MapAsset);

				TArray<FString> MapTests;
				FAssetDataTagMapSharedView::FFindTagResult MapAssetNamesResult = MapAsset.TagsAndValues.FindTag(bEditorOnlyTests ? TEXT("TestNamesEditor") : TEXT("TestNames"));

				if (MapAssetNamesResult.IsSet())
				{
					MapAssetNamesResult.GetValue().ParseIntoArray(MapTests, TEXT(";"), true);
				}

#if WITH_EDITOR
				// Also append external functional test actors
				if (ULevel::GetIsLevelUsingExternalActorsFromAsset(MapAsset))
				{
					const FString LevelExternalActorsPath = ULevel::GetExternalActorsPath(MapPackageName);

					// Do a synchronous scan of the level external actors path.			
					AssetRegistry.ScanPathsSynchronous({ LevelExternalActorsPath }, /*bForceRescan*/false, /*bIgnoreDenyListScanFilters*/false);

					FARFilter ActorsFilter;
					ActorsFilter.bRecursivePaths = true;
					ActorsFilter.bIncludeOnlyOnDiskAssets = true;
					ActorsFilter.PackagePaths.Add(*LevelExternalActorsPath);

					TArray<FAssetData> ActorList;
					AssetRegistry.GetAssets(ActorsFilter, ActorList);

					for (const FAssetData& ActorAsset : ActorList)
					{
						FAssetDataTagMapSharedView::FFindTagResult ActorTestName = ActorAsset.TagsAndValues.FindTag(bEditorOnlyTests ? TEXT("TestNameEditor") : TEXT("TestName"));

						if (ActorTestName.IsSet())
						{
							MapTests.Add(ActorTestName.GetValue());
						}
					}
				}
#endif
				if (!MapTests.IsEmpty())
				{
					for (const FString& MapTest : MapTests)
					{
						FString Remainder;
						FString BeautifulTestName;
						FString RealTestName;
						FString TestTags;

						if (MapTest.Split(TEXT("|"), &BeautifulTestName, &Remainder))
						{
							if (!Remainder.Split(TEXT("|"), &RealTestName, &TestTags))
							{
								// split fails when loading an old format test-asset without tags
								RealTestName = Remainder; // tags remain empty
							}

							FString FullBeautifiedName(PartialSuiteName + TEXT(".") + *BeautifulTestName);
							FString TestCommand(MapAssetPath + TEXT(";") + MapPackageName + TEXT(";") + *RealTestName);
							OutTestInfo.Add(FFunctionalTestInfo(FullBeautifiedName, TestCommand, TestTags));
							OutTestMapAssets.AddUnique(MapAssetPath);
						}
					}
				}
				else if (!bEditorOnlyTests && MapAsset.AssetName.ToString().StartsWith(TEXT("FTEST_")))
				{
					// add legacy functional test maps which autoplay
					FString FullBeautifiedName(MapAsset.AssetName.ToString());
					FString TestCommand(MapAssetPath + TEXT(";") + MapPackageName);
					OutTestInfo.Add(FFunctionalTestInfo(FullBeautifiedName, TestCommand, "")); // Tags not supported
					OutTestMapAssets.AddUnique(MapAssetPath);
				}
			}
		}
	}
}

void FFunctionalTestingModule::SetManager(class UFunctionalTestingManager* NewManager)
{
	TestManager = NewManager;
}

UFunctionalTestingManager* FFunctionalTestingModule::GetCurrentManager()
{
	return TestManager.Get();
}

bool FFunctionalTestingModule::IsRunning() const
{
	return TestManager.IsValid() && TestManager->IsRunning();
}

bool FFunctionalTestingModule::IsFinished() const
{
	return (!TestManager.IsValid() || TestManager->IsFinished());
}

void FFunctionalTestingModule::MarkPendingActivation()
{
	bPendingActivation = true;
}

bool FFunctionalTestingModule::IsActivationPending() const
{
	return bPendingActivation;
}

void FFunctionalTestingModule::SetLooping(const bool bLoop)
{
	if (TestManager.IsValid())
	{
		TestManager->SetLooped(bLoop);
	}
}

UWorld* FFunctionalTestingModule::GetTestWorld()
{
#if WITH_EDITOR
	const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
	for (const FWorldContext& Context : WorldContexts)
	{
		if (Context.World() != nullptr)
		{
			if (Context.WorldType == EWorldType::PIE /*&& Context.PIEInstance == 0*/)
			{
				return Context.World();
			}
			
			if (Context.WorldType == EWorldType::Game)
			{
				return Context.World();
			}
		}
	}
#endif

	return GWorld;
}

void FFunctionalTestingModule::RunAllTestsOnMap(bool bClearLog, bool bRunLooped)
{
	if (UWorld* TestWorld = GetTestWorld())
	{
		bPendingActivation = false;
		if (UFunctionalTestingManager::RunAllFunctionalTests(TestWorld, bClearLog, bRunLooped) == false)
		{
			UE_LOGF(LogFunctionalTest, Error, "No functional testing script on map.");
		}
	}
}

void FFunctionalTestingModule::RunTestOnMap(const FString& TestName, bool bClearLog, bool bRunLooped)
{
	if (UWorld* TestWorld = GetTestWorld())
	{
		bPendingActivation = false;
		if (UFunctionalTestingManager::RunAllFunctionalTests(TestWorld, bClearLog, bRunLooped, TestName) == false)
		{
			UE_LOGF(LogFunctionalTest, Error, "No functional testing script on map.");
		}
	}
}


//////////////////////////////////////////////////////////////////////////
// Exec
//////////////////////////////////////////////////////////////////////////

static bool FuncTestExec(UWorld* InWorld, const TCHAR* Command, FOutputDevice& Ar)
{
	if (FParse::Command(&Command, TEXT("ftest")))
	{
		if (FParse::Command(&Command, TEXT("start")))
		{
			const bool bLooped = FParse::Command(&Command, TEXT("loop"));

			//instead of allowing straight use of the functional test framework, this should go through the automation framework and kick off one of the Editor/Client functional tests

			IFunctionalTestingModule& Module = IFunctionalTestingModule::Get();
			if (!Module.IsRunning() && !Module.IsActivationPending())
			{
				Module.RunAllTestsOnMap(/*bClearLog=*/true, bLooped);
			}
		}
		return true;
	}
	return false;
}

FStaticSelfRegisteringExec FuncTestExecRegistration(FuncTestExec);


IMPLEMENT_MODULE( FFunctionalTestingModule, FunctionalTesting );

#undef LOCTEXT_NAMESPACE
