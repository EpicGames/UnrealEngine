// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalUIScreenshotTest.h"

#include "Engine/GameViewportClient.h"
#include "AutomationBlueprintFunctionLibrary.h"
#include "Camera/CameraComponent.h"
#include "Camera/PlayerCameraManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Misc/AutomationTest.h"
#include "Widgets/SViewport.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Slate/SceneViewport.h"
#include "Slate/SlateViewportProvider.h"
#include "Slate/WidgetRenderer.h"
#include "TextureResource.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FunctionalUIScreenshotTest)

AFunctionalUIScreenshotTest::AFunctionalUIScreenshotTest( const FObjectInitializer& ObjectInitializer )
	: AScreenshotFunctionalTestBase(ObjectInitializer)
{
	WidgetLocation = EWidgetTestAppearLocation::Viewport;
	bHideDebugCanvas = true;
}

/**
 * Get pixel format and color space of a backbuffer. Do nothing if the viewport doesn't
 * render into backbuffer directly
 * @InViewport - the viewport to get backbuffer from
 * @OutPixelFormat - pixel format of the backbuffer
 * @OutIsSRGB - whether the backbuffer stores pixels in sRGB space
 */ 
void GetBackbufferInfo(const FViewport* InViewport, EPixelFormat* OutPixelFormat, bool* OutIsSRGB)
{
	ENQUEUE_RENDER_COMMAND(GetBackbufferFormatCmd)(
		[InViewport, OutPixelFormat, OutIsSRGB](FRHICommandListImmediate& RHICmdList)
	{
		FTextureRHIRef BackbufferTexture = InViewport->GetRenderTargetTexture();
		check(BackbufferTexture.IsValid());
		*OutPixelFormat = BackbufferTexture->GetFormat();
		*OutIsSRGB = (BackbufferTexture->GetFlags() & TexCreate_SRGB) == TexCreate_SRGB;
	});
	FlushRenderingCommands();
}

void AFunctionalUIScreenshotTest::PrepareTest()
{
	// Resize viewport to screenshot size
	Super::PrepareTest();

	// Hide all debug info
	if (bHideDebugCanvas)
	{
		if (IConsoleVariable* CVarDebugCanvasVisible = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.GameLayer.DebugCanvasVisible")))
		{
			PreviousDebugCanvasVisible = CVarDebugCanvasVisible->GetBool();
			CVarDebugCanvasVisible->Set(false);
		}
	}

	TSharedPtr<SViewport> GameViewportWidget = GEngine->GameViewport->GetGameViewportWidget();
	check(GameViewportWidget.IsValid());

	// If render directly to backbuffer, just read from backbuffer
	if (!GameViewportWidget->ShouldRenderDirectly())
	{
		// Resize screenshot render target to have the same size as the game viewport. Also
		// make sure they have the same data format (pixel format, color space, etc.) if possible
		const FSceneViewport* GameViewport = GEngine->GameViewport->GetGameViewport();
		FIntPoint ScreenshotSize = GameViewport->GetSizeXY();
		EPixelFormat PixelFormat = PF_A2B10G10R10;
		bool bIsSRGB = false;
		GetBackbufferInfo(GameViewport, &PixelFormat, &bIsSRGB);

		if (!ScreenshotRT)
		{
			ScreenshotRT = NewObject<UTextureRenderTarget2D>(this);
		}
		ScreenshotRT->ClearColor = FLinearColor::Transparent;
		ScreenshotRT->InitCustomFormat(ScreenshotSize.X, ScreenshotSize.Y, PixelFormat, !bIsSRGB);
	}

	// Spawn the widget
	APlayerController* PlayerController = UGameplayStatics::GetPlayerController(GetWorld(), 0);
	SpawnedWidget = CreateWidget<UUserWidget>(PlayerController, WidgetClass);
	
	if (SpawnedWidget)
	{
		if (WidgetLocation == EWidgetTestAppearLocation::Viewport)
		{
			SpawnedWidget->AddToViewport();
		}
		else
		{
			// Add to the game viewport and restrain the widget within
			// owning player's sub-rect
			SpawnedWidget->AddToPlayerScreen();
		}

		SpawnedWidget->SetVisibility(ESlateVisibility::HitTestInvisible);
	}
	NumTickPassed = 0;
	if (IConsoleVariable* CVarFixedDeltaTime = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.UseFixedDeltaTime")))
	{
		bWasPreviouslyUsingFixedDeltaTime = CVarFixedDeltaTime->GetBool();
		PreviousFixedDeltaTime = FSlateApplication::GetFixedDeltaTime();
		FSlateApplication::SetFixedDeltaTime(TestFixedDeltaTime);
		CVarFixedDeltaTime->SetWithCurrentPriority(true);
	}

	UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot();
}

void AFunctionalUIScreenshotTest::OnScreenshotTakenAndCompared()
{
	if (SpawnedWidget)
	{
		SpawnedWidget->RemoveFromParent();
	}

	// Restore viewport size and finish the test
	Super::OnScreenshotTakenAndCompared();

	// Restore the debug text
	if (PreviousDebugCanvasVisible.IsSet())
	{
		if (IConsoleVariable* CVarDebugCanvasVisible = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.GameLayer.DebugCanvasVisible")))
		{
			CVarDebugCanvasVisible->Set(PreviousDebugCanvasVisible.GetValue());
			PreviousDebugCanvasVisible.Reset();
		}
	}
}

void AFunctionalUIScreenshotTest::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	// Restore the debug text
	if (PreviousDebugCanvasVisible.IsSet())
	{
		if (IConsoleVariable* CVarDebugCanvasVisible = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.GameLayer.DebugCanvasVisible")))
		{
			CVarDebugCanvasVisible->Set(PreviousDebugCanvasVisible.GetValue());
			PreviousDebugCanvasVisible.Reset();
		}
	}

	if (IConsoleVariable* CVarFixedDeltaTime = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.UseFixedDeltaTime")))
	{
		CVarFixedDeltaTime->SetWithCurrentPriority(bWasPreviouslyUsingFixedDeltaTime);
		FSlateApplication::SetFixedDeltaTime(PreviousFixedDeltaTime);
	}
	NumTickPassed = 0;
}

bool  AFunctionalUIScreenshotTest::IsReady_Implementation()
{
	if (NumTickPassed * FSlateApplication::GetFixedDeltaTime() >= ScreenshotOptions.Delay)
	{
		return NumTickPassed > ScreenshotOptions.FrameDelay;
	}
	return false;
}

void  AFunctionalUIScreenshotTest::Tick(float DeltaSeconds)
{
	NumTickPassed += 1;

	if (Future.IsValid() && Future.IsReady())
	{
		const FScreenshotData& Data = Future.Get();
		GEngine->GameViewport->OnScreenshotCaptured().Broadcast(Data.Size.X, Data.Size.Y, Data.ColorData);
		Future = {};

		if (Handle.IsValid())
		{
			FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().Remove(Handle);
			Handle = {};
		}
	}

	Super::Tick(DeltaSeconds);
}

void AFunctionalUIScreenshotTest::ReadBackBufferData(TPromise<FScreenshotData>& Promise, FRHITexture* Texture)
{
	FScreenshotData Data;
	Data.Size = Texture->GetSizeXY();

	FRHICommandListImmediate::Get().ReadSurfaceData(
		Texture,
		FIntRect(FIntPoint(0, 0), Data.Size),
		Data.ColorData,
		FReadSurfaceDataFlags());

	// For UI, we only care about what the final image looks like. So don't compare alpha channel.
	for (int32 Idx = 0; Idx < Data.ColorData.Num(); ++Idx)
	{
		Data.ColorData[Idx].A = 0xff;
	}

	Promise.SetValue(Data);
}

void AFunctionalUIScreenshotTest::RequestScreenshot()
{
	// Register a handler to UGameViewportClient::OnScreenshotCaptured
	Super::RequestScreenshot();

	UGameViewportClient* GameViewportClient = GEngine->GameViewport;
	TSharedPtr<SViewport> ViewportWidget = GameViewportClient->GetGameViewportWidget();

	if (ViewportWidget.IsValid())
	{
		TPromise<FScreenshotData> Promise;
		Future = Promise.GetFuture();

		if (ViewportWidget->ShouldRenderDirectly())
		{
			// Flush the render thread to make sure the below lambda callback doesn't run too early.
			FlushRenderingCommands();

			// Hook the Slate back buffer presentation so we can capture the image data.
			// The result is handed back to the game thread via the TFuture/TPromise, and picked up in Tick().
			Handle = FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().AddLambda(
				[
					Viewport = ViewportWidget->GetViewportInterface().Pin(),
					Promise = MakeShared<TPromise<FScreenshotData>>(MoveTemp(Promise)).ToSharedPtr()
				](SWindow& Window, ISlateViewportProvider& ViewportProvider) mutable
			{
				if (!Promise || Window.GetViewport() != Viewport)
				{
					return;
				}

				ReadBackBufferData(*Promise, ViewportProvider.GetBackBufferResource());

				// Release the promise to make sure this lambda only runs once.
				Promise = nullptr;
			});
		}
		else
		{
			FIntPoint ScreenshotSize = GameViewportClient->GetGameViewport()->GetSizeXY();

			// Draw the game viewport (overlaid with the widget to screenshot) to our ScreenshotRT.
			// Need to do this manually because the game viewport doesn't have a valid FViewportRHIRef when rendering to a separate render target
			FWidgetRenderer* WidgetRenderer = new FWidgetRenderer(true, false);
			WidgetRenderer->DrawWidget(ScreenshotRT, ViewportWidget.ToSharedRef(), ViewportWidget->GetCachedGeometry().Scale, ScreenshotSize, 0.f);

			ENQUEUE_RENDER_COMMAND(ReadScreenshotRTCmd)(
				[Promise = MoveTemp(Promise), Texture = ScreenshotRT](FRHICommandListImmediate& RHICmdList) mutable
			{
				ReadBackBufferData(Promise, static_cast<FTextureRenderTarget2DResource*>(Texture->GetRenderTargetResource())->GetTextureRHI());
			});

			BeginCleanup(WidgetRenderer);
		}
	}
}
