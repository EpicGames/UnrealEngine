// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroundTruthData.h"

#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetData.h"

#if WITH_EDITOR
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"
#include "SourceControlHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroundTruthData)

DEFINE_LOG_CATEGORY_STATIC(GroundTruthLog, Log, Log)

UGroundTruthData::UGroundTruthData()
	: bResetGroundTruth(false)
{
}

bool UGroundTruthData::CanModify() const
{
	return ObjectData == nullptr;
}

UObject* UGroundTruthData::LoadObject()
{
	UE_LOGF(GroundTruthLog, Log, "Loaded Ground Truth, '%ls'.", *GetPathName());
	
	return ObjectData;
}

void UGroundTruthData::SaveObject(UObject* GroundTruth)
{
	FAssetData GroundTruthAssetData(this);

	UPackage* GroundTruthPackage = GetOutermost();
	FString GroundTruthPackageName = GroundTruthAssetData.PackageName.ToString();

#if WITH_EDITOR
	if (!CanModify())
	{
		UE_LOGF(GroundTruthLog, Warning, "Ground Truth, '%ls' already set, unable to save changes.  Open and use bResetGroundTruth to reset the ground truth.", *GroundTruthPackageName);
		return;
	}

	if (GroundTruth == nullptr)
	{
		UE_LOGF(GroundTruthLog, Error, "Ground Truth, '%ls' can not store a null object.", *GroundTruthPackageName);
		return;
	}

	if (GIsBuildMachine)
	{
		UE_LOGF(GroundTruthLog, Error, "Ground Truth, '%ls' can not be modified on the build machine.", *GroundTruthPackageName);
		return;
	}

	if (ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), GroundTruthPackage);
		SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), GroundTruthPackage);
	}

	// There is an edge case for Actors where calling `Rename` will transfer ownership from the level to the GroundTruthData and will remove the Actor from the current level
	// To avoid any issues this process causes, we want to duplicate our object with a transient package before performing the rename which will change the ownership from the transient package to the GroundTruthData
	ObjectData = DuplicateObject(GroundTruth, nullptr);
	ObjectData->Rename(nullptr, this, REN_SkipComponentRegWork);
	MarkPackageDirty();

	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;
	SaveArgs.SaveFlags = SAVE_NoError;
	if (!UPackage::SavePackage(GroundTruthPackage, nullptr,
		*FPackageName::LongPackageNameToFilename(GroundTruthPackageName, FPackageName::GetAssetPackageExtension()),
		SaveArgs))
	{
		UE_LOGF(GroundTruthLog, Error, "Failed to save ground truth data! %ls", *GroundTruthPackageName);
	}

	UE_LOGF(GroundTruthLog, Log, "Saved Ground Truth, '%ls'.", *GroundTruthPackageName);

#else
	UE_LOGF(GroundTruthLog, Error, "Can't save ground truth data outside of the editor, '%ls'.", *GroundTruthPackageName);
#endif
}

#if WITH_EDITOR

void UGroundTruthData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UGroundTruthData, bResetGroundTruth))
	{
		bResetGroundTruth = false;

		if (ObjectData)
		{
			ObjectData->Rename(nullptr, GetTransientPackage());
			ObjectData = nullptr;
		}

		MarkPackageDirty();
	}
}

#endif

void UGroundTruthData::ResetObject()
{
	#if WITH_EDITOR

		if (ObjectData)
		{
			ObjectData->Rename(nullptr, GetTransientPackage());
			ObjectData = nullptr;

			MarkPackageDirty();
		}

	#else
		FAssetData GroundTruthAssetData(this);
		FString GroundTruthPackageName = GroundTruthAssetData.PackageName.ToString();

		UE_LOGF(GroundTruthLog, Error, "Can't reset ground truth data outside of the editor, '%ls'.", *GroundTruthPackageName);
	#endif
}	