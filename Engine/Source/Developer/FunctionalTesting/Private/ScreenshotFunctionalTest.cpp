// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScreenshotFunctionalTest.h"

#include "Engine/GameViewportClient.h"
#include "AutomationBlueprintFunctionLibrary.h"
#include "Camera/CameraComponent.h"
#include "Camera/PlayerCameraManager.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Misc/AutomationTest.h"
#include "HighResScreenshot.h"
#include "UnrealClient.h"
#include "Slate/SceneViewport.h"
#include "UObject/AutomationObjectVersion.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Tests/AutomationCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScreenshotFunctionalTest)

AScreenshotFunctionalTest::AScreenshotFunctionalTest( const FObjectInitializer& ObjectInitializer )
	: AScreenshotFunctionalTestBase(ObjectInitializer)
	, bCameraCutOnScreenshotPrep(true)
	, RequestedVariants(false, EVariantType::Num)
	, bVariantQueued(false)
{
}

void AScreenshotFunctionalTest::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FAutomationObjectVersion::GUID);

	if (Ar.CustomVer(FAutomationObjectVersion::GUID) < FAutomationObjectVersion::DefaultToScreenshotCameraCutAndFixedTonemapping)
	{
		bCameraCutOnScreenshotPrep = true;
	}
}

bool AScreenshotFunctionalTest::RunTest(const TArray<FString>& Params)
{
	// Set up which variants are enabled
	check(!RequestedVariants.Contains(true));

	if (FAutomationTestFramework::NeedPerformStereoTestVariants())
	{
		RequestedVariants[EVariantType::Baseline] = !FAutomationTestFramework::NeedUseLightweightStereoTestVariants(); // Skip baseline if lightweight variants are active
		RequestedVariants[EVariantType::ViewRectOffset] = true;

	}
	else
	{
		RequestedVariants[EVariantType::Baseline] = true;
		RequestedVariants[EVariantType::ViewRectOffset] = false;
	}

	// Set up first variant
	SetupNextVariant();

	// This will call PrepareTest()
	return Super::RunTest(); 
}

void AScreenshotFunctionalTest::PrepareTest()
{
	// Pre-prep flush to allow rendering to temporary targets and other test resources
	UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot();

	Super::PrepareTest();

	// Apply a camera cut if requested
	if (bCameraCutOnScreenshotPrep)
	{
		APlayerController* PlayerController = UGameplayStatics::GetPlayerController(GetWorld(), 0);

		if (PlayerController && PlayerController->PlayerCameraManager)
		{
			PlayerController->PlayerCameraManager->SetGameCameraCutThisFrame();
			if (ScreenshotCamera)
			{
				ScreenshotCamera->NotifyCameraCut();
			}
		}
	}

	// Post-prep flush deal with any temporary resources allocated during prep before the main test
	UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot();
}

void AScreenshotFunctionalTest::RequestScreenshot()
{
	Super::RequestScreenshot();

	if(IsMobilePlatform(GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel]) && !GEngine->IsStereoscopic3D())
	{
		// For mobile, use the high res screenshot API to ensure a fixed resolution screenshot is produced.
		// This means screenshot comparisons can compare with the output from any device.
		FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();
		FIntPoint Size = UAutomationBlueprintFunctionLibrary::GetAutomationScreenshotSize(ScreenshotOptions);
		if (Config.SetResolution(Size.X, Size.Y, 1.0f))
		{
			UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
			check(GameViewportClient);
			GameViewportClient->GetGameViewport()->TakeHighResScreenShot();
		}
	}
	else
	{
		// Screenshots in Unreal Engine work in this way:
		// 1. Call FScreenshotRequest::RequestScreenshot to ask the system to take a screenshot. The screenshot
		//    will have the same resolution as the current viewport;
		// 2. Register a callback to UGameViewportClient::OnScreenshotCaptured() delegate. The call back will be
		//    called with screenshot pixel data when the shot is taken;
		// 3. Wait till the next frame or call FSceneViewport::Invalidate to force a redraw. Screenshot is not
		//    taken until next draw where UGameViewportClient::ProcessScreenshots or
		//    FEditorViewportClient::ProcessScreenshots is called to read pixels back from the viewport. It also
		//    trigger the callback function registered in step 2.

		FIntRect ViewRect = FIntRect(); // Passing an empty ViewRect defaults to including to the entire render target in the screenshot

		if (ScreenshotOptions.bAllowEmulatingSplitscreen)
		{
			static const TConsoleVariableData<int32>* CVarEmulateSplitscreenMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ScreenshotTest.EmulateSplitscreenMode"));
			const int32 EmulateSplitscreenMode = CVarEmulateSplitscreenMode ? CVarEmulateSplitscreenMode->GetValueOnAnyThread() : 0;
			const FIntPoint Size = UAutomationBlueprintFunctionLibrary::GetAutomationScreenshotSize(ScreenshotOptions);	

			switch (EmulateSplitscreenMode)
			{
				case 0:
					break;
				case 1:	// Horizontal
					ViewRect = FIntRect(Size.X, 0, Size.X * 2, Size.Y);
					break;
				case 2:	// Vertical
					ViewRect = FIntRect(0, Size.Y, Size.X, Size.Y * 2);
					break;
				case 3:	// Diagonal
					ViewRect = FIntRect(Size, Size * 2);
					break;
				default:
					// Already hit ensure in AScreenshotFunctionalTestBase::PrepareForScreenshot()
					break;
			}

			checkf(ViewRect.Size() == Size || ViewRect.IsEmpty(), TEXT("Screenshot view rect should either match the expected automation screenshot size, or be empty (which requests a screenshot of the entire render target)."));
		}
		
		FScreenshotRequest::RequestScreenshot(TEXT(""), /*bInShowUI*/ false, /*bAddUniqueSuffix*/ true, /*bHdrScreenshot*/ false, ViewRect);
	}
}

void AScreenshotFunctionalTest::OnScreenShotCaptured(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData)
{
	UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
	check(GameViewportClient);

	GameViewportClient->OnScreenshotCaptured().RemoveAll(this);

#if WITH_AUTOMATION_TESTS
	if (!IsRunning())
	{
		// Don't send the data if the test is no longer running.
		return;
	}

	const FString Context = AutomationCommon::GetWorldContext(GetWorld());

	TArray<uint8> CapturedFrameTrace = AutomationCommon::CaptureFrameTrace(Context, TestLabel);

	FAutomationScreenshotData Data = UAutomationBlueprintFunctionLibrary::BuildScreenshotData(Context, TestLabel, InSizeX, InSizeY);

	// Copy the relevant data into the metadata for the screenshot.
	Data.bHasComparisonRules = true;
	Data.ToleranceRed = ScreenshotOptions.ToleranceAmount.Red;
	Data.ToleranceGreen = ScreenshotOptions.ToleranceAmount.Green;
	Data.ToleranceBlue = ScreenshotOptions.ToleranceAmount.Blue;
	Data.ToleranceAlpha = ScreenshotOptions.ToleranceAmount.Alpha;
	Data.ToleranceMinBrightness = ScreenshotOptions.ToleranceAmount.MinBrightness;
	Data.ToleranceMaxBrightness = ScreenshotOptions.ToleranceAmount.MaxBrightness;
	Data.bIgnoreAntiAliasing = ScreenshotOptions.bIgnoreAntiAliasing;
	Data.bIgnoreColors = ScreenshotOptions.bIgnoreColors;
	Data.MaximumLocalError = ScreenshotOptions.MaximumLocalError;
	Data.MaximumGlobalError = ScreenshotOptions.MaximumGlobalError;

	// Add the notes
	Data.Notes = Notes;

	// If variant in use, pass on the name, then restore settings since capture is done
	Data.VariantName = CurrentVariantName;
	if (VariantRestoreCommand)
	{
		GEngine->Exec(nullptr, VariantRestoreCommand);
		VariantRestoreCommand = nullptr;
	}

	if (GIsAutomationTesting)
	{
		FAutomationTestFramework::Get().OnScreenshotCompared.AddUObject(this, &AScreenshotFunctionalTest::OnComparisonComplete);
	}

	FAutomationTestFramework::Get().OnScreenshotAndTraceCaptured().ExecuteIfBound(InImageData, CapturedFrameTrace, Data);

	UE_LOGF(LogScreenshotFunctionalTest, Log, "Screenshot captured as %ls", *Data.ScreenshotPath);
#endif
}

void AScreenshotFunctionalTest::OnScreenshotTakenAndCompared()
{
	FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest();
	bool bSkipRemainingVariants = FAutomationTestFramework::Get().NeedUseLightweightStereoTestVariants() && (!CurrentTest || CurrentTest->HasAnyErrors());

	// If we aren't skipping further variants due to a failure, and we still have some remaining, queue the next
	if (!bSkipRemainingVariants && SetupNextVariant())
	{
		ReprepareTest();
	}

	// Otherwise finish
	else
	{
		Super::OnScreenshotTakenAndCompared();
	}
}

bool AScreenshotFunctionalTest::SetupNextVariant()
{
	// Find first remaining requested variant
	int32 Index = RequestedVariants.Find(true);

	if (Index == INDEX_NONE)
	{
		return false;
	}
	else
	{
		// Remove variant from requested variants and set it up
		RequestedVariants[Index] = false;
		SetupVariant(static_cast<EVariantType>(Index));
		return true;
	}
}

void AScreenshotFunctionalTest::SetupVariant(EVariantType VariantType)
{
	// Get info for requested variant
	static const TArray<FVariantInfo> VariantInfoArray =
	{
		FVariantInfo{ TEXT(""), nullptr, nullptr }, // Baseline
		FVariantInfo{ TEXT("ViewRectOffset"), TEXT("r.Test.ViewRectOffset 5"), TEXT("r.Test.ViewRectOffset 0") } // ViewRectOffset
	};

	const FVariantInfo* VariantInfo = &VariantInfoArray[VariantType];

	// Set up variant
	if (VariantInfo->SetupCommand)
	{
		GEngine->Exec(nullptr, VariantInfo->SetupCommand);
	}

	// Save variant name and command needed to restore in OnScreenShotCaptured
	CurrentVariantName = VariantInfo->Name;
	if (VariantInfo->RestoreCommand)
	{
		VariantRestoreCommand = VariantInfo->RestoreCommand;
	}
}

void AScreenshotFunctionalTest::ReprepareTest()
{
	// Required to reset TSR sequences
	OnTestFinished.Broadcast();

	RestoreViewSettings();

	// Rather than re-use RunFrame/RunTime, use a seprarate set for subsequent variants
	// A screenshot will be requested once the given delay has elapsed
	VariantFrame = GFrameNumber;
	VariantTime = (float)GetWorld()->GetTimeSeconds();
	bVariantQueued = true;

	PrepareTest();
}

// TSR sequences override this to rely on an internal blueprint instead, but it will still use our Tick() implementation
// Because we call OnTestFinished.Broadcast() in ReprepareTest(), the TSR BP frame counter is reset properly
bool AScreenshotFunctionalTest::IsReady_Implementation()
{
	if (bVariantQueued)
	{
		if ((GetWorld()->GetTimeSeconds() - VariantTime) > ScreenshotOptions.Delay)
		{
			return int32(GFrameNumber - VariantFrame) > ScreenshotOptions.FrameDelay;
		}

		return false;
	}
	else
	{
		return Super::IsReady_Implementation();
	}
}

void AScreenshotFunctionalTest::Tick(float DeltaSeconds)
{
	// This section takes over from the main loop to kick off subsequent variants
	if (bVariantQueued)
	{
		if (IsReady())
		{
			bVariantQueued = false;
			StartTest();
		}

	}
	else
	{
		if (PreparationTimeLimit > 0.f && TotalTime > PreparationTimeLimit)
		{
			OnTimeout();
		}
	}

	Super::Tick(DeltaSeconds);
}