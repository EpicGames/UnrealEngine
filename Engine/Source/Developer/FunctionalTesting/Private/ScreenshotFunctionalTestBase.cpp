// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScreenshotFunctionalTestBase.h"

#include "Engine/GameViewportClient.h"
#include "AutomationBlueprintFunctionLibrary.h"
#include "Camera/CameraComponent.h"
#include "Camera/PlayerCameraManager.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/Engine.h"
#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif
#include "EngineGlobals.h"
#include "Misc/AutomationTest.h"
#include "Slate/SceneViewport.h"
#include "RenderingThread.h"
#include "Tests/AutomationCommon.h"
#include "Logging/LogMacros.h"
#include "UObject/AutomationObjectVersion.h"
#include "RenderGraphBuilder.h"
#include "Engine/LocalPlayer.h"
#include "RenderUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScreenshotFunctionalTestBase)

#define	WITH_EDITOR_AUTOMATION_TESTS	(WITH_EDITOR && WITH_AUTOMATION_TESTS)

static TAutoConsoleVariable<int32> GDumpGPUDumpOnScreenshotTest(
	TEXT("r.DumpGPU.DumpOnScreenshotTest"), 0,
	TEXT("Allows to filter the tree when using r.DumpGPU command, the pattern match is case sensitive."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarEmulateSplitscreenMode(
	TEXT("r.ScreenshotTest.EmulateSplitscreenMode"), 0,
	TEXT("Allows automtically emulating splitscreen rendering for all screenshot functional tests. Useful for catching splitscreen exclusive rendering issues.\n")
	TEXT("When active and bAllowEmulatingSplitscreen=true for a test, two full size views will be rendered, and screenshot comparison tests will capture only the second.\n")
	TEXT("Tests with existing ground truths should pass if there are no splitscreen rendering issues.")
	TEXT("If a test is failing in splitscreen due to differences that do not impact correctness (such as changes in noise patterns), you can exempt it by setting \"Allow Emulating Splitscren\" to false under the Screenshot Options.\n")
	TEXT("0 - Off\n")
	TEXT("1 - Horizontal: Viewports are side by side\n")
	TEXT("2 - Vertical: Viewports are at the top and bottom\n")
	TEXT("3 - Diagonal: Viewports are in the upper left and lower right, leaving gaps in the other corners of the render target"),
	ECVF_Default);

AScreenshotFunctionalTestBase::AScreenshotFunctionalTestBase(const FObjectInitializer& ObjectInitializer)
	: AFunctionalTest(ObjectInitializer)
	, ScreenshotOptions(EComparisonTolerance::Low)
	, bNeedsViewSettingsRestore(false)
	, bNeedsViewportRestore(false)
	, bScreenshotCompleted(false)
{
	ScreenshotCamera = CreateDefaultSubobject<UCameraComponent>(TEXT("Camera"));
	ScreenshotCamera->SetupAttachment(RootComponent);
	ScreenshotCamera->bLockToHmd = false;

#if WITH_AUTOMATION_TESTS
	ScreenshotEnvSetup = MakeShareable(new FAutomationTestScreenshotEnvSetup());
#endif
}

void AScreenshotFunctionalTestBase::PrepareTest()
{
	Super::PrepareTest();

	UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
	check(GameViewportClient);

	APlayerController* PlayerController = UGameplayStatics::GetPlayerController(GameViewportClient->GetWorld(), 0);
	if (PlayerController)
	{
		// Make sure the camera target is not auto managed
		PlayerController->bAutoManageActiveCameraTarget = false;
		PlayerController->SetViewTarget(this, FViewTargetTransitionParams());

		if (ScreenshotOptions.bAllowEmulatingSplitscreen)
		{
			const int32 EmulateSplitscreenMode = CVarEmulateSplitscreenMode.GetValueOnAnyThread();
			if (EmulateSplitscreenMode >= 1 && EmulateSplitscreenMode <= 3)
			{
				PlayerController->GetLocalPlayer()->bEmulateSplitscreen = true;
			}
		}
	}

	PrepareForScreenshot();
}

bool AScreenshotFunctionalTestBase::IsReady_Implementation()
{
	if ((GetWorld()->GetTimeSeconds() - RunTime) > ScreenshotOptions.Delay)
	{
		return int32(GFrameNumber - RunFrame) > ScreenshotOptions.FrameDelay;
	}

	return false;
}

void AScreenshotFunctionalTestBase::StartTest()
{
	Super::StartTest();

	FAutomationTestFramework::Get().OnScreenshotTakenAndCompared.AddUObject(this, &AScreenshotFunctionalTestBase::OnScreenshotTakenAndCompared);

	RequestScreenshot();
}

void AScreenshotFunctionalTestBase::OnScreenshotTakenAndCompared()
{
	bScreenshotCompleted = true;

	FinishTest(EFunctionalTestResult::Succeeded, TEXT(""));
}

void AScreenshotFunctionalTestBase::FinishTest(EFunctionalTestResult TestResult, const FString& Message)
{
	if (!IsReady() ||  bScreenshotCompleted)
	{
		RestoreViewSettings();

		FAutomationTestFramework::Get().OnScreenshotTakenAndCompared.RemoveAll(this);

		Super::FinishTest(TestResult, Message);
	}
	else if (TestResult == EFunctionalTestResult::Error
				|| TestResult == EFunctionalTestResult::Failed
				|| TestResult == EFunctionalTestResult::Invalid)
	{
		AddError(Message);
	}
}

void AScreenshotFunctionalTestBase::PrepareForScreenshot()
{
	UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
	check(GameViewportClient);
	check(IsInGameThread());
	check(!bNeedsViewSettingsRestore && !bNeedsViewportRestore);

	bScreenshotCompleted = false;

#if WITH_AUTOMATION_TESTS
	bool bApplyScreenshotSettings = true;
	FSceneViewport* GameViewport = GameViewportClient->GetGameViewport();

#if WITH_EDITOR_AUTOMATION_TESTS
	// In the editor we can only attempt to resize a standalone viewport
	UWorld* World = GameViewportClient->GetWorld();
	UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);	

	const bool bIsPIEViewport = GameViewport->IsPlayInEditorViewport();
	const bool bIsNewViewport = World && EditorEngine && EditorEngine->WorldIsPIEInNewViewport(World);

	bApplyScreenshotSettings = !bIsPIEViewport || bIsNewViewport;
#endif	

	if (bApplyScreenshotSettings)
	{
		ScreenshotEnvSetup->Setup(GameViewportClient->GetWorld(), ScreenshotOptions);
		FlushRenderingCommands();
		bNeedsViewSettingsRestore = true;

		// Some platforms (such as consoles) require fixed width/height 
		// back-buffers so we cannot adjust the viewport size
		if (!FPlatformProperties::HasFixedResolution())
		{
			ViewportRestoreSize = GameViewport->GetSize();
			FIntPoint ScreenshotViewportSize = UAutomationBlueprintFunctionLibrary::GetAutomationScreenshotSize(ScreenshotOptions);

			if (ScreenshotOptions.bAllowEmulatingSplitscreen)
			{
				const int32 EmulateSplitscreenMode = CVarEmulateSplitscreenMode->GetInt();

				if (EmulateSplitscreenMode >= 1 && EmulateSplitscreenMode <= 3)
				{
					FIntPoint QuantizedScreenshotViewportSize;
					QuantizeSceneBufferSize(ScreenshotViewportSize, QuantizedScreenshotViewportSize);

					if (ScreenshotViewportSize != QuantizedScreenshotViewportSize)
					{
						UE_LOGF(LogScreenshotFunctionalTest, Warning, "Screenshot size %ls does not match quantized screenshot size %ls, emulated splitscreen rendering may behave incorrectly.",
							*ScreenshotViewportSize.ToString(), *QuantizedScreenshotViewportSize.ToString());
					}
				}

				switch (EmulateSplitscreenMode)
				{
					case 0:
						break;
					case 1:	// Horizontal
						ScreenshotViewportSize.X *= 2;
						break;
					case 2:	// Vertical
						ScreenshotViewportSize.Y *= 2;
						break;
					case 3:	// Diagonal
						ScreenshotViewportSize *= 2;
						break;
					default:
						ensureMsgf(false, TEXT("Selected r.ScreenshotTest.EmulateSplitscreenMode of %d is invalid. Defaulting to 0 (off)."), EmulateSplitscreenMode);
						break;
				}
			}

			GameViewport->SetViewportSize(ScreenshotViewportSize.X, ScreenshotViewportSize.Y);
			bNeedsViewportRestore = true;
		}
	}

#if WITH_DUMPGPU
	// Reset render target extent to reduce size of the dumpGPU.
	if (GDumpGPUDumpOnScreenshotTest.GetValueOnGameThread() != 0)
	{
		FlushRenderingCommands();
		UKismetSystemLibrary::ExecuteConsoleCommand(GameViewportClient->GetWorld(), TEXT("r.ResetRenderTargetsExtent"), nullptr);
	}
#endif
#endif
}

void AScreenshotFunctionalTestBase::OnScreenShotCaptured(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData)
{
	UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
	check(GameViewportClient);

	GameViewportClient->OnScreenshotCaptured().RemoveAll(this);

#if WITH_AUTOMATION_TESTS
	if (!IsRunning())
	{
		// Don't send the data if the test is no longer running.
		return;
	}

	const FString Context = AutomationCommon::GetWorldContext(GetWorld());

	TArray<uint8> CapturedFrameTrace = AutomationCommon::CaptureFrameTrace(Context, TestLabel);

	FAutomationScreenshotData Data = UAutomationBlueprintFunctionLibrary::BuildScreenshotData(Context, TestLabel, InSizeX, InSizeY);

	// Copy the relevant data into the metadata for the screenshot.
	Data.bHasComparisonRules = true;
	Data.ToleranceRed = ScreenshotOptions.ToleranceAmount.Red;
	Data.ToleranceGreen = ScreenshotOptions.ToleranceAmount.Green;
	Data.ToleranceBlue = ScreenshotOptions.ToleranceAmount.Blue;
	Data.ToleranceAlpha = ScreenshotOptions.ToleranceAmount.Alpha;
	Data.ToleranceMinBrightness = ScreenshotOptions.ToleranceAmount.MinBrightness;
	Data.ToleranceMaxBrightness = ScreenshotOptions.ToleranceAmount.MaxBrightness;
	Data.bIgnoreAntiAliasing = ScreenshotOptions.bIgnoreAntiAliasing;
	Data.bIgnoreColors = ScreenshotOptions.bIgnoreColors;
	Data.MaximumLocalError = ScreenshotOptions.MaximumLocalError;
	Data.MaximumGlobalError = ScreenshotOptions.MaximumGlobalError;

	// Add the notes
	Data.Notes = Notes;

	if (GIsAutomationTesting)
	{
		FAutomationTestFramework::Get().OnScreenshotCompared.AddUObject(this, &AScreenshotFunctionalTestBase::OnComparisonComplete);
	}

	FAutomationTestFramework::Get().OnScreenshotAndTraceCaptured().ExecuteIfBound(InImageData, CapturedFrameTrace, Data);

	UE_LOGF(LogScreenshotFunctionalTest, Log, "Screenshot captured as %ls", *Data.ScreenshotPath);
#endif
}

void AScreenshotFunctionalTestBase::RequestScreenshot()
{
	check(IsInGameThread());
	UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
	check(GameViewportClient);

	// Make sure any screenshot request has been processed
	FlushRenderingCommands();

	GameViewportClient->OnScreenshotCaptured().AddUObject(this, &AScreenshotFunctionalTestBase::OnScreenShotCaptured);

#if WITH_AUTOMATION_TESTS && WITH_DUMPGPU
	if (GDumpGPUDumpOnScreenshotTest.GetValueOnGameThread() != 0)
	{
		FRDGBuilder::BeginResourceDump(TEXT(""), TEXT("ScreenshotTest"));
	}
#endif
}

void AScreenshotFunctionalTestBase::OnComparisonComplete(const FAutomationScreenshotCompareResults& CompareResults)
{
	FAutomationTestFramework::Get().OnScreenshotCompared.RemoveAll(this);

	if(!bIsRunning)
	{
		return;
	}

	if (FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest())
	{
		CurrentTest->AddEvent(CompareResults.ToAutomationEvent());
	}

	FAutomationTestFramework::Get().NotifyScreenshotTakenAndCompared();
}

void AScreenshotFunctionalTestBase::RestoreViewSettings()
{
	UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
	check(GameViewportClient && GameViewportClient->GetGameViewport());
	check(IsInGameThread());
	
#if WITH_AUTOMATION_TESTS
	if (bNeedsViewSettingsRestore)
	{
		ScreenshotEnvSetup->Restore();
	}

	if (!FPlatformProperties::HasFixedResolution() && bNeedsViewportRestore)
	{	
		FSceneViewport* GameViewport = GameViewportClient->GetGameViewport();
		GameViewport->SetViewportSize(ViewportRestoreSize.X, ViewportRestoreSize.Y);
	}
#endif

	bNeedsViewSettingsRestore = false;
	bNeedsViewportRestore = false;
}

void AScreenshotFunctionalTestBase::OnTimeout()
{
	// If the test timed out, make sure the screenshot comparison is cancelled.
	bScreenshotCompleted = true;

	Super::OnTimeout();
}

#if WITH_EDITOR

bool AScreenshotFunctionalTestBase::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FAutomationScreenshotOptions, ToleranceAmount))
		{
			bIsEditable = ScreenshotOptions.Tolerance == EComparisonTolerance::Custom;
		}
		else if (PropertyName == TEXT("ObservationPoint"))
		{
			// You can't ever observe from anywhere but the camera on the screenshot test.
			bIsEditable = false;
		}
	}

	return bIsEditable;
}

void AScreenshotFunctionalTestBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FAutomationScreenshotOptions, Tolerance))
		{
			ScreenshotOptions.SetToleranceAmounts(ScreenshotOptions.Tolerance);
		}
	}
}

#endif

void AScreenshotFunctionalTestBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FAutomationObjectVersion::GUID);

	if (Ar.CustomVer(FAutomationObjectVersion::GUID) < FAutomationObjectVersion::DefaultToScreenshotCameraCutAndFixedTonemapping)
	{
		ScreenshotOptions.bDisableTonemapping = true;
	}
}


