// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeScreenshotAfterTimeLatentAction.h"
#include "AutomationBlueprintFunctionLibrary.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Misc/AutomationTest.h"

FTakeScreenshotAfterTimeLatentAction::FTakeScreenshotAfterTimeLatentAction(const FLatentActionInfo& LatentInfo, const FString& InScreenshotName, const FString& InNotes, FAutomationScreenshotOptions InOptions)
	: ExecutionFunction(LatentInfo.ExecutionFunction)
	, OutputLink(LatentInfo.Linkage)
	, CallbackTarget(LatentInfo.CallbackTarget)
	, ScreenshotName(InScreenshotName)
	, Notes(InNotes)
	, SecondsRemaining(InOptions.Delay)
	, FinishedLoading(false)
	, IssuedScreenshotCapture(false)
	, TakenScreenshot(false)
	, Options(InOptions)
{

}

FTakeScreenshotAfterTimeLatentAction::~FTakeScreenshotAfterTimeLatentAction()
{
	FAutomationTestFramework::Get().OnScreenshotTakenAndCompared.RemoveAll(this);
}

void FTakeScreenshotAfterTimeLatentAction::OnScreenshotTakenAndCompared()
{
	TakenScreenshot = true;
}

void FTakeScreenshotAfterTimeLatentAction::UpdateOperation(FLatentResponse& Response)
{
	if (!FinishedLoading)
	{
		UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot();
		FinishedLoading = true;
	}

	if ( !TakenScreenshot )
	{
		if ( !IssuedScreenshotCapture )
		{
			UAutomationBlueprintFunctionLibrary::FinishLoadingBeforeScreenshot();

			SecondsRemaining -= Response.ElapsedTime();
			if ( SecondsRemaining <= 0.0f )
			{
				UObject* Caller = CallbackTarget.IsValid() ? CallbackTarget.Get() : nullptr;
				FAutomationTestFramework::Get().OnScreenshotTakenAndCompared.AddRaw(this, &FTakeScreenshotAfterTimeLatentAction::OnScreenshotTakenAndCompared);

				if ( UAutomationBlueprintFunctionLibrary::TakeAutomationScreenshotInternal(Caller, ScreenshotName, Notes, Options) )
				{
					IssuedScreenshotCapture = true;
				}
				else
				{
					//TODO LOG FAILED SCREENSHOT
					TakenScreenshot = true;
				}
			}
		}
	}
	else
	{
		FAutomationTestFramework::Get().OnScreenshotTakenAndCompared.RemoveAll(this);
		Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
	}
}

#if WITH_EDITOR
FString FTakeScreenshotAfterTimeLatentAction::GetDescription() const
{
	return FString::Printf(TEXT("Take screenshot named %s after %f seconds"), *ScreenshotName, SecondsRemaining);
}
#endif






FWaitForScreenshotComparisonLatentAction::FWaitForScreenshotComparisonLatentAction(const FLatentActionInfo& LatentInfo)
	: ExecutionFunction(LatentInfo.ExecutionFunction)
	, OutputLink(LatentInfo.Linkage)
	, CallbackTarget(LatentInfo.CallbackTarget)
	, TakenScreenshot(false)
{
	FAutomationTestFramework::Get().OnScreenshotTakenAndCompared.AddRaw(this, &FWaitForScreenshotComparisonLatentAction::OnScreenshotTakenAndCompared);
}

FWaitForScreenshotComparisonLatentAction::~FWaitForScreenshotComparisonLatentAction()
{
	FAutomationTestFramework::Get().OnScreenshotTakenAndCompared.RemoveAll(this);
}

void FWaitForScreenshotComparisonLatentAction::OnScreenshotTakenAndCompared()
{
	TakenScreenshot = true;
}

void FWaitForScreenshotComparisonLatentAction::UpdateOperation(FLatentResponse& Response)
{
	if ( TakenScreenshot )
	{
		FAutomationTestFramework::Get().OnScreenshotTakenAndCompared.RemoveAll(this);
		Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
	}
}

#if WITH_EDITOR
FString FWaitForScreenshotComparisonLatentAction::GetDescription() const
{
	return FString(TEXT(""));
}
#endif
