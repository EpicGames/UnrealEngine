// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "Engine/Engine.h"
#include "AssetRegistry/AssetData.h"
#include "FunctionalTestingModule.h"
#include "EngineGlobals.h"
#include "Tests/AutomationCommon.h"
#include "FunctionalTestBase.h"
#include "FunctionalTest.h"
#include "FunctionalTestingHelper.h"
#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "Engine/Level.h" // could be forward declare, but we need to know ULevel is a UObject
#include "FunctionalTest.h"
#include "Tests/AutomationEditorCommon.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "FunctionalTesting"

DEFINE_LOG_CATEGORY_STATIC(LogFunctionalTesting, Log, All);

class FClientFunctionalTestingMapsBase : public FFunctionalTestBase
{
public:
	FClientFunctionalTestingMapsBase(const FString& InName, const bool bInComplexTask)
		: FFunctionalTestBase(InName, bInComplexTask)
	{
	}

	// Project.Maps.Client Functional Testing
	// Project.Maps.Functional Tests

	static void ParseTestMapInfo(const FString& Parameters, FString& MapObjectPath, FString& MapPackageName, FString& MapTestName)
	{
		TArray<FString> ParamArray;
		Parameters.ParseIntoArray(ParamArray, TEXT(";"), true);

		MapObjectPath = ParamArray[0];
		MapPackageName = ParamArray[1];
		MapTestName = (ParamArray.Num() > 2) ? ParamArray[2] : TEXT("");
	}

	// @todo this is a temporary solution. Once we know how to get test's hands on a proper world
	// this function should be redone/removed
	static UWorld* GetAnyGameWorld()
	{
		UWorld* TestWorld = nullptr;
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{
			if (((Context.WorldType == EWorldType::PIE) || (Context.WorldType == EWorldType::Game)) && (Context.World() != NULL))
			{
				TestWorld = Context.World();
				break;
			}
		}

		return TestWorld;
	}

	virtual FString GetTestOpenCommand(const FString& Parameters) const override
	{
		FString MapObjectPath, MapPackageName, MapTestName;
		ParseTestMapInfo(Parameters, MapObjectPath, MapPackageName, MapTestName);

		return FString::Printf(TEXT("Automate.OpenMapAndFocusActor %s %s"), *MapObjectPath, *MapTestName);
	}

	virtual FString GetTestAssetPath(const FString& Parameters) const override
	{
		FString MapObjectPath, MapPackageName, MapTestName;
		ParseTestMapInfo(Parameters, MapObjectPath, MapPackageName, MapTestName);

		return MapObjectPath;
	}

	/** 
	 * Requests a enumeration of all maps to be loaded
	 */
	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const override
	{
		bool bEditorOnlyTests = !(GetTestFlags() & EAutomationTestFlags::ClientContext);
		TArray<FString> MapAssetsUnused;
		TArray<FFunctionalTestInfo> AllTestInfo;
		IFunctionalTestingModule::Get().GetMapTests(bEditorOnlyTests, AllTestInfo, MapAssetsUnused);
		FAutomationTestFramework& Framework = FAutomationTestFramework::Get();
		for (const FFunctionalTestInfo& TestInfo : AllTestInfo)
		{
			OutBeautifiedNames.Add(TestInfo.BeautifiedName);
			OutTestCommands.Add(TestInfo.TestCommand);
			if (!TestInfo.TestTags.IsEmpty())
			{
				// Register new tags
				Framework.RegisterComplexAutomationTestTags(this, TestInfo.BeautifiedName, TestInfo.TestTags);
			}
		}
	}

	/**
	 * Execute the loading of each map and performance captures
	 *
	 * @param Parameters - Should specify which map name to load
	 * @return	TRUE if the test was successful, FALSE otherwise
	 */
	virtual bool RunTest(const FString& Parameters) override
	{
		FString MapObjectPath, MapPackageName, MapTestName;
		ParseTestMapInfo(Parameters, MapObjectPath, MapPackageName, MapTestName);

		bool bCanProceed = false;

		IFunctionalTestingModule::Get().MarkPendingActivation();

		// Always reset these, even though tests should do the same
		SetLogErrorAndWarningHandlingToDefault();

		UWorld* TestWorld = GetAnyGameWorld();
		if (TestWorld && TestWorld->GetMapName() == MapPackageName)
		{
			// Map is already loaded.
			bCanProceed = true;
		}
		else
		{
			bCanProceed = AutomationOpenMap(MapPackageName);
		}

		if (bCanProceed)
		{
			if (MapTestName.IsEmpty())
			{
				ADD_LATENT_AUTOMATION_COMMAND(FStartFTestsOnMap());
			}
			else
			{
				ADD_LATENT_AUTOMATION_COMMAND(FStartFTestOnMap(MapTestName));
			}

			return true;
		}

		/// FAutomationTestFramework::GetInstance().UnregisterAutomationTest

		//	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(1.f));
		//  ADD_LATENT_AUTOMATION_COMMAND(FExitGameCommand);

		UE_LOGF(LogFunctionalTest, Error, "Failed to start the %ls map (possibly due to BP compilation issues)", *MapPackageName);
		return false;
	}

};

// Runtime tests
IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST(FClientFunctionalTestingMapsRuntime, FClientFunctionalTestingMapsBase, "Project.Functional Tests", (EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter))

void FClientFunctionalTestingMapsRuntime::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	FClientFunctionalTestingMapsBase::GetTests(OutBeautifiedNames, OutTestCommands);
}

bool FClientFunctionalTestingMapsRuntime::RunTest(const FString& Parameters)
{
	return FClientFunctionalTestingMapsBase::RunTest(Parameters);
}

// Editor only tests
IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST(FClientFunctionalTestingMapsEditor, FClientFunctionalTestingMapsBase, "Project.Functional Tests", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter))

void FClientFunctionalTestingMapsEditor::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	FClientFunctionalTestingMapsBase::GetTests(OutBeautifiedNames, OutTestCommands);
}

bool FClientFunctionalTestingMapsEditor::RunTest(const FString& Parameters)
{
#if WITH_EDITOR	
	extern UNREALED_API UEditorEngine* GEditor;
	FString MapObjectPath, MapPackageName, MapTestName;
	ParseTestMapInfo(Parameters, MapObjectPath, MapPackageName, MapTestName);
	
	// check for a world to reuse:
	const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
	const UWorld* WorldToUse = nullptr;
	for (const FWorldContext& Context : WorldContexts)
	{
		// don't reuse editor world, as it may have accumulated edits
		if (((Context.WorldType == EWorldType::PIE) || (Context.WorldType == EWorldType::Game)) && Context.LastURL.Map == MapPackageName )
		{
			if(const UWorld* World = Context.World())
			{
				WorldToUse = World;
				break;
			}
		}
	}

	if(WorldToUse)
	{
		// we have a world to use - but does it have the test we now need to run?
		AFunctionalTest* Test = FindObjectFast<AFunctionalTest>(WorldToUse->PersistentLevel, FName(MapTestName));
		if(Test && (!IsEditorOnlyObject(Test) || Test->IsEditorOnlyLoadedInPIE() || !IsEditorOnlyObject(Test->GetClass())))
		{
			// it does, just use base test running logic, as it will be faster than
			// reopening the map:
			return FClientFunctionalTestingMapsBase::RunTest(Parameters);
		}
	}
	
	IFunctionalTestingModule::Get().MarkPendingActivation();

	// Always reset these, even though tests should do the same
	SetLogErrorAndWarningHandlingToDefault();

	AddCommand(new FOpenEditorForAssetCommand(MapPackageName));
	AddCommand(new FFunctionLatentCommand([]() {
		// wait for editor world
		return GEditor->GetEditorWorldContext().World() != nullptr;
	}));
	
	// if the actor wants a pie world, start PIE:
	if(!MapTestName.IsEmpty())
	{
		AddCommand(new FFunctionLatentCommand([MapTestName, MapPackageName]() {
			UWorld* World = GEditor->GetEditorWorldContext().World();
			AFunctionalTest* Test = FindObjectFast<AFunctionalTest>(World->PersistentLevel, FName(MapTestName));
			// Actors that indicate they are editor only but want IsNonPIEEditorOnly or are of a class
			// that is not editor only should run in PIE:
			if(ensure(Test) && (Test->IsEditorOnlyLoadedInPIE() || !IsEditorOnlyObject(Test->GetClass())))
			{
				// request PIE, as this test actor claims it supports PIE:
				const bool bCanProceed = AutomationOpenMap(MapPackageName);
				if(!bCanProceed)
				{
					UE_LOGF(LogFunctionalTest, Error, "Failed to start the %ls map (possibly due to BP compilation issues)", *MapPackageName);
				}
			}
			return true;
		}));
	}

	// run the test - note that FStartFTestOnMap is going to 
	// create two more latent commands in its update function,
	// meaning any more latent commands you add will run
	// before FStartFTestOnMap actually does anything (as it
	// will simply enqueue commands after your command)
	ADD_LATENT_AUTOMATION_COMMAND(FStartFTestOnMap(MapTestName));
	// !!! ANY COMMANDS ADDED HERE WILL RUN BEFORE FStartFTestOnMap !!!
	return true;
#else
	// This is likely unreachable, but because this is tag based we can't guarantee that no one is using 
	// EAutomationTestFlags::EditorContext outside of editor
	return FClientFunctionalTestingMapsBase::RunTest(Parameters);
#endif
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_DEV_AUTOMATION_TESTS
