// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceQueryTestResults.h"
#include "FunctionalTest.h"
#include "Components/PrimitiveComponent.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TraceQueryTestResults)

FString FTraceQueryTestNames::ToString() const
{
	return FString::Printf(TEXT("Component:%s, Actor:%s, PhysicalMaterial:%s"), *ComponentName.ToString(), *ActorName.ToString(), *PhysicalMaterialName.ToString());
}

FString FTraceQueryTestResultsInnerMost::ToString() const
{
	FString OutSingle = FString::Printf(TEXT("----SingleResult:%d %s {%s}"), bSingleResult, *SingleNames.ToString(), *SingleHit.ToString());
	FString OutMulti = FString::Printf(TEXT("\n----MultiResult:%d"), bMultiResult);
	
	for(int32 Counter = 0; Counter < MultiHits.Num(); ++Counter)
	{
		OutMulti.Append(FString::Printf(TEXT("\n----[%d] %s {%s}"), Counter, *MultiNames[Counter].ToString(),  *MultiHits[Counter].ToString()));
	}

	return OutSingle.Append(OutMulti);
}

FString FTraceQueryTestResultsInner::ToString(const FTraceChannelTestBatchOptions& BatchOptions) const
{
	FString LineResultsStr = BatchOptions.bLineTrace ? FString::Printf(TEXT("--LineResults:\n%s\n\n"), *LineResults.ToString()) : "";
	FString SphereResultsStr = BatchOptions.bSphereTrace ? FString::Printf(TEXT("--SphereResults:\n%s\n\n"), *SphereResults.ToString()) : "";
	FString CapsuleResultsStr = BatchOptions.bCapsuleTrace ? FString::Printf(TEXT("--CapsuleResults:\n%s\n\n"), *CapsuleResults.ToString()) : "";
	FString BoxResultsStr = BatchOptions.bBoxTrace ? FString::Printf(TEXT("--BoxResults:\n%s\n\n"), *BoxResults.ToString()) : "";

	return FString::Printf(TEXT("%s%s%s%s"), *LineResultsStr, *SphereResultsStr, *CapsuleResultsStr, *BoxResultsStr);
}

FString UTraceQueryTestResults::ToString()
{
	CaptureNames();

	FString ChannelResultsStr = BatchOptions.bChannelTrace ? FString::Printf(TEXT("ChannelResults:\n%s\n\n"), *ChannelResults.ToString(BatchOptions)) : "";
	FString ObjectResultsStr = BatchOptions.bObjectsTrace ? FString::Printf(TEXT("ObjectResults:\n%s\n\n"), *ObjectResults.ToString(BatchOptions)) : "";
	FString ProfileResultsStr = BatchOptions.bProfileTrace ? FString::Printf(TEXT("ProfileResults:\n%s\n\n"), *ProfileResults.ToString(BatchOptions)) : "";
	
	return FString::Printf(TEXT("%s%s%s"), *ChannelResultsStr, *ObjectResultsStr, *ProfileResultsStr);
}

void CaptureNameHelper(FTraceQueryTestNames& Names, FHitResult& HitResult)
{
	UPrimitiveComponent* HitComp = HitResult.GetComponent();
	UPhysicalMaterial* PhysMat = HitResult.PhysMaterial.Get();

	Names.ComponentName = HitComp ? HitComp->GetFName() : NAME_None;
	Names.ActorName = HitResult.HitObjectHandle.IsValid() ? HitResult.HitObjectHandle.GetFName() : NAME_None;
	Names.PhysicalMaterialName = PhysMat ? PhysMat->GetFName() : NAME_None;
}

void FTraceQueryTestResultsInnerMost::CaptureNames()
{
	CaptureNameHelper(SingleNames, SingleHit);

	MultiNames.Reset();
	MultiNames.AddZeroed(MultiHits.Num());

	for(int32 HitIdx = 0; HitIdx < MultiHits.Num(); ++HitIdx)
	{
		CaptureNameHelper(MultiNames[HitIdx], MultiHits[HitIdx]);
	}
}

void FTraceQueryTestResultsInner::CaptureNames()
{
	LineResults.CaptureNames();
	SphereResults.CaptureNames();
	CapsuleResults.CaptureNames();
	BoxResults.CaptureNames();
}

void UTraceQueryTestResults::CaptureNames()
{
	ChannelResults.CaptureNames();
	ObjectResults.CaptureNames();
	ProfileResults.CaptureNames();
}

FString FTraceChannelTestBatchOptions::ToString() const
{
	return FString::Printf(TEXT("bLineTrace:%d, bSphereTrace:%d, bCapsuleTrace:%d, bBoxTrace:%d, bChannelTrace:%d, bObjectsTrace:%d, bProfileTrace:%d"), bLineTrace, bSphereTrace, bCapsuleTrace, bBoxTrace, bChannelTrace, bObjectsTrace, bProfileTrace);
}

#define TEST_IMPL(X, Type) FunctionalTest.AssertEqual_##Type(Actual.X, Expected.X, FString::Printf(TEXT("%s:"#X), *What), ContextObject)
#define TEST_IMPL_TOLERANCE(X, Type) FunctionalTest.AssertEqual_##Type(Actual.X, Expected.X, FString::Printf(TEXT("%s:"#X), *What), KINDA_SMALL_NUMBER, ContextObject)
#define TEST_BOOL(X) TEST_IMPL(X, Bool)
#define TEST_INT(X) TEST_IMPL(X, Int)
#define TEST_NAME(X) TEST_IMPL(X, Name)
#define TEST_FLOAT(X) TEST_IMPL_TOLERANCE(X, Float)
#define TEST_VECTOR(X) TEST_IMPL_TOLERANCE(X, Vector)

bool HelperAssertNamesEqual(const FTraceQueryTestNames& Actual, const FTraceQueryTestNames& Expected, const FString& What, const UObject* ContextObject, AFunctionalTest& FunctionalTest)
{
	bool bResults[] = {
		TEST_NAME(ComponentName),
		TEST_NAME(ActorName),
		TEST_NAME(PhysicalMaterialName)
	};
	
	for (bool bSuccess : bResults)
	{
		if (!bSuccess)
		{
			return false;
		}
	}

	return true;
}

bool HelperAssertFHitResultEqual(const FHitResult& Actual, const FHitResult& Expected, const FString& What, const UObject* ContextObject, AFunctionalTest& FunctionalTest)
{
	bool bResults[] = {
		TEST_BOOL(bBlockingHit),
		TEST_BOOL(bStartPenetrating),
		TEST_FLOAT(Time),
		TEST_FLOAT(Distance),
		TEST_VECTOR(Location),
		TEST_VECTOR(ImpactPoint),
		TEST_VECTOR(Normal),
		TEST_VECTOR(ImpactNormal),
		TEST_VECTOR(TraceStart),
		TEST_VECTOR(TraceEnd),
		TEST_FLOAT(PenetrationDepth),
		TEST_INT(Item),
		TEST_NAME(BoneName),
		TEST_INT(FaceIndex)
	};
	
	for(bool bSuccess : bResults)
	{
		if(!bSuccess)
		{
			return false;
		}
	}

	return true;
}

#undef TEST_IMPL
#undef TEST_IMPL_TOLERANCE
#undef TEST_BOOL
#undef TEST_INT
#undef TEST_FLOAT
#undef TEST_VECTOR

bool FTraceQueryTestResultsInnerMost::AssertEqual(const FTraceQueryTestResultsInnerMost& Expected, const FString& What, const UObject* ContextObject, AFunctionalTest& FunctionalTest) const
{
	bool bSuccess = true;
	if(FunctionalTest.AssertEqual_Bool(bSingleResult, Expected.bSingleResult, FString::Printf(TEXT("%s:bSingleResult"), *What), ContextObject))
	{
		bSuccess = HelperAssertFHitResultEqual(SingleHit, Expected.SingleHit, FString::Printf(TEXT("%s_SingleHit"), *What), ContextObject, FunctionalTest);
		bSuccess &= HelperAssertNamesEqual(SingleNames, Expected.SingleNames, FString::Printf(TEXT("%s_SingleHit"), *What), ContextObject, FunctionalTest);
	}
	else
	{
		bSuccess = false;
	}

	if (FunctionalTest.AssertEqual_Bool(bMultiResult, Expected.bMultiResult, FString::Printf(TEXT("%s:bMultiResult"), *What), ContextObject))
	{
		if(FunctionalTest.AssertValue_Int(MultiHits.Num(), EComparisonMethod::Equal_To, Expected.MultiHits.Num(), FString::Printf(TEXT("%s:MultiHitsNum"), *What), ContextObject))
		{
			for (int32 HitIdx = 0; HitIdx < MultiHits.Num(); ++HitIdx)
			{
				bSuccess &= HelperAssertFHitResultEqual(MultiHits[HitIdx], Expected.MultiHits[HitIdx], FString::Printf(TEXT("%s_MultiHit[%d]"), *What, HitIdx), ContextObject, FunctionalTest);
				bSuccess &= HelperAssertNamesEqual(MultiNames[HitIdx], Expected.MultiNames[HitIdx], FString::Printf(TEXT("%s_MultiHitHit[%d]"), *What, HitIdx), ContextObject, FunctionalTest);
			}
		}
		else
		{
			bSuccess = false;
		}
	}
	else
	{
		bSuccess = false;
	}

	return bSuccess;
}

bool FTraceQueryTestResultsInner::AssertEqual(const FTraceQueryTestResultsInner& Expected, const FString& What, const UObject* ContextObject, AFunctionalTest& FunctionalTest) const
{
	const bool bLineResults = LineResults.AssertEqual(Expected.LineResults, FString::Printf(TEXT("%s_Line"), *What), ContextObject, FunctionalTest);
	const bool bSphereResults = SphereResults.AssertEqual(Expected.SphereResults, FString::Printf(TEXT("%s_Sphere"), *What), ContextObject, FunctionalTest);
	const bool bCapsuleResults = CapsuleResults.AssertEqual(Expected.CapsuleResults, FString::Printf(TEXT("%s_Capsule"), *What), ContextObject, FunctionalTest);
	const bool bBoxResults = BoxResults.AssertEqual(Expected.BoxResults, FString::Printf(TEXT("%s_Box"), *What), ContextObject, FunctionalTest);

	return bLineResults && bSphereResults && bCapsuleResults && bBoxResults;
}

bool UTraceQueryTestResults::AssertEqual(const UTraceQueryTestResults* Expected, const FString& What, const UObject* ContextObject, AFunctionalTest& FunctionalTest) const
{
	if(Expected == nullptr)
	{
		FunctionalTest.LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' 'Expected != nullptr' for context '%s'"), *What, ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else if(Expected->BatchOptions != BatchOptions)
	{
		FunctionalTest.LogStep(ELogVerbosity::Error, FString::Printf(TEXT("Expected '%s' to be {%s} but it was {%s} for context '%s'"), *What, *Expected->BatchOptions.ToString(), *BatchOptions.ToString(), ContextObject ? *ContextObject->GetName() : TEXT("")));
		return false;
	}
	else
	{
		//Purposely do all 3 so we get errors for as much as possible
		const bool bChannelResults = ChannelResults.AssertEqual(Expected->ChannelResults, FString::Printf(TEXT("%s_Channel"), *What), ContextObject, FunctionalTest);
		const bool bObjectResults = ObjectResults.AssertEqual(Expected->ObjectResults, FString::Printf(TEXT("%s_Object"), *What), ContextObject, FunctionalTest);
		const bool bProfileResults = ProfileResults.AssertEqual(Expected->ProfileResults, FString::Printf(TEXT("%s_Profile"), *What), ContextObject, FunctionalTest);
		if( bChannelResults && bObjectResults && bProfileResults)
		{
			FunctionalTest.LogStep(ELogVerbosity::Log, FString::Printf(TEXT("TraceQueryTestResults assertion passed (%s)"), *What));
			return true;
		}
		else
		{
			FunctionalTest.LogStep(ELogVerbosity::Error, FString::Printf(TEXT("'%s' comparison failed for context '%s'"), *What, ContextObject ? *ContextObject->GetName() : TEXT("")));
			return false;
		}
	}
}


