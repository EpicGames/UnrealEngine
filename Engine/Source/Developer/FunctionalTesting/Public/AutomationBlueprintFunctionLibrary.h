// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AutomationScreenshotOptions.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/LatentActionManager.h"
#include "HAL/IConsoleManager.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/AutomationTest.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectMacros.h"
#include "AutomationBlueprintFunctionLibrary.generated.h"

#define UE_API FUNCTIONALTESTING_API

class ACameraActor;

/**
 * FAutomationTaskStatusBase - abstract class for task status
 */
class FAutomationTaskStatusBase
{
public:
	virtual ~FAutomationTaskStatusBase() = default;

	bool IsDone() const { return Done; };
	virtual void SetDone() { Done = true; };

protected:
	bool Done = false;
};

/**
 * UAutomationEditorTask
 */
UCLASS(MinimalAPI, BlueprintType, Transient)
class UAutomationEditorTask : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UAutomationEditorTask() = default;

	/** Query if the Editor task is done  */
	UFUNCTION(BlueprintCallable, Category = "Automation")
	UE_API bool IsTaskDone() const;

	/** Query if a task was setup */
	UFUNCTION(BlueprintCallable, Category = "Automation")
	UE_API bool IsValidTask() const;

	UE_API void BindTask(TUniquePtr<FAutomationTaskStatusBase> inTask);

private:
	TUniquePtr<FAutomationTaskStatusBase> Task;
};

USTRUCT(BlueprintType)
struct FAutomationWaitForLoadingOptions
{
	GENERATED_BODY()

public:

	UPROPERTY(BlueprintReadWrite, Category = "Automation")
	bool WaitForReplicationToSettle = false;
};

/**
 * 
 */
UCLASS(MinimalAPI, meta=(ScriptName="AutomationLibrary"))
class UAutomationBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	
public:
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void FinishLoadingBeforeScreenshot();

	static UE_API bool TakeAutomationScreenshotInternal(UObject* WorldContextObject, const FString& ScreenShotName, const FString& Notes, FAutomationScreenshotOptions Options);

	static UE_API FAutomationScreenshotData BuildScreenshotData(const FString& MapOrContext, const FString& ScreenShotName, int32 Width, int32 Height);
	static UE_API FAutomationScreenshotData BuildScreenshotData(UWorld* InWorld, const FString& ScreenShotName, int32 Width, int32 Height);

	static UE_API FIntPoint GetAutomationScreenshotSize(const FAutomationScreenshotOptions& Options);

	/**
	 * Takes a screenshot of the game's viewport.  Does not capture any UI.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (Latent, HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject", LatentInfo = "LatentInfo", Name = "" ))
	static UE_API void TakeAutomationScreenshot(UObject* WorldContextObject, FLatentActionInfo LatentInfo, const FString& Name, const FString& Notes, const FAutomationScreenshotOptions& Options);

	/**
	 * Takes a screenshot of the game's viewport, from a particular camera actors POV.  Does not capture any UI.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (Latent, HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject", LatentInfo = "LatentInfo", NameOverride = "" ))
	static UE_API void TakeAutomationScreenshotAtCamera(UObject* WorldContextObject, FLatentActionInfo LatentInfo, ACameraActor* Camera, const FString& NameOverride, const FString& Notes, const FAutomationScreenshotOptions& Options);

	/**
	 * 
	 */
	static UE_API bool TakeAutomationScreenshotOfUI_Immediate(UObject* WorldContextObject, const FString& Name, const FAutomationScreenshotOptions& Options);

	UFUNCTION(BlueprintCallable, Category = "Automation", meta = ( Latent, HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject", LatentInfo = "LatentInfo", NameOverride = "" ))
	static UE_API void TakeAutomationScreenshotOfUI(UObject* WorldContextObject, FLatentActionInfo LatentInfo, const FString& Name, const FAutomationScreenshotOptions& Options);

	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	static UE_API void EnableStatGroup(UObject* WorldContextObject, FName GroupName);

	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	static UE_API void DisableStatGroup(UObject* WorldContextObject, FName GroupName);

	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API float GetStatIncAverage(FName StatName);

	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API float GetStatIncMax(FName StatName);

	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API float GetStatExcAverage(FName StatName);

	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API float GetStatExcMax(FName StatName);

	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API float GetStatCallCount(FName StatName);

	/**
	 * Lets you know if any automated tests are running, or are about to run and the automation system is spinning up tests.
	 */
	UFUNCTION(BlueprintPure, Category="Automation")
	static UE_API bool AreAutomatedTestsRunning();

	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (Latent, HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject", LatentInfo = "LatentInfo"))
	static UE_API void AutomationWaitForLoading(UObject* WorldContextObject, FLatentActionInfo LatentInfo, FAutomationWaitForLoadingOptions Options);

	/**
	* take high res screenshot in editor.
	*/
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (AdvancedDisplay="Camera, bMaskEnabled, bCaptureHDR, ComparisonTolerance, ComparisonNotes, bForceGameView"))
	static UE_API UAutomationEditorTask* TakeHighResScreenshot(int32 ResX, int32 ResY, FString Filename, ACameraActor* Camera = nullptr, bool bMaskEnabled = false, bool bCaptureHDR = false, EComparisonTolerance ComparisonTolerance = EComparisonTolerance::Low, FString ComparisonNotes = TEXT(""), float Delay = 0.0, bool bForceGameView = true);

	/**
	* request image comparison.
	* @param ImageFilePath	Absolute path to the image location. All 8bit RGBA channels supported formats by the engine are accepted.
	* @param ComparisonName	Optional name for the comparison, by default the basename of ImageFilePath is used
	* @return				True if comparison was successfully enqueued
	*/
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (AdvancedDisplay = "ComparisonName, ComparisonTolerance, ComparisonNotes", HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	static UE_API bool CompareImageAgainstReference(FString ImageFilePath, FString ComparisonName = TEXT(""), EComparisonTolerance ComparisonTolerance = EComparisonTolerance::Low, FString ComparisonNotes = TEXT(""), UObject* WorldContextObject = nullptr);

	/**
	* Add Telemetry data to currently running automated test.
	*/
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (AdvancedDisplay = "Context"))
	static UE_API void AddTestTelemetryData(FString DataPoint, float Measurement, FString Context = TEXT(""));

	/**
	* Set Telemetry data storage name of currently running automated test.
	*/
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void SetTestTelemetryStorage(FString StorageName);

	/**
	 * 
	 */
	UFUNCTION(BlueprintPure, Category="Automation")
	static UE_API FAutomationScreenshotOptions GetDefaultScreenshotOptionsForGameplay(EComparisonTolerance Tolerance = EComparisonTolerance::Low, float Delay = 0.2);

	/**
	 * 
	 */
	UFUNCTION(BlueprintPure, Category="Automation")
	static UE_API FAutomationScreenshotOptions GetDefaultScreenshotOptionsForRendering(EComparisonTolerance Tolerance = EComparisonTolerance::Low, float Delay = 0.2);

	/**
	 * Mute the report of log error and warning matching a pattern during an automated test. Treat the pattern as regex by default.
	 * @param ExpectedPatternString	Expects a Regex pattern.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (AdvancedDisplay = "Occurrences, ExactMatch, IsRegex"))
	static UE_API void AddExpectedLogError(FString ExpectedPatternString, int32 Occurrences = 1, bool ExactMatch = false, bool IsRegex = true);

	/**
	 * Mute the report of log error and warning matching a plain string during an automated test
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (AdvancedDisplay = "Occurrences, ExactMatch"))
	static UE_API void AddExpectedPlainLogError(FString ExpectedString, int32 Occurrences = 1, bool ExactMatch = false);

	/**
	 * Expect a specific log message to match a pattern during an automated test regardless of its verbosity. Treat the pattern as regex by default.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (AdvancedDisplay = "Occurrences, ExactMatch, IsRegex"))
	static UE_API void AddExpectedLogMessage(FString ExpectedPatternString, int32 Occurrences = 1, bool ExactMatch = false, bool IsRegex = true);

	/**
	 * Expect a specific log message to match a plain string during an automated test regardless of its verbosity
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (AdvancedDisplay = "Occurrences, ExactMatch"))
	static UE_API void AddExpectedPlainLogMessage(FString ExpectedString, int32 Occurrences = 1, bool ExactMatch = false);

	/**
	 * Sets all other settings based on an overall value
	 * @param Value 0:Cinematic, 1:Epic...etc.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	static UE_API void SetScalabilityQualityLevelRelativeToMax(UObject* WorldContextObject, int32 Value = 1);

	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	static UE_API void SetScalabilityQualityToEpic(UObject* WorldContextObject);

	UFUNCTION(BlueprintCallable, Category = "Automation", meta = (HidePin = "WorldContextObject", DefaultToSelf = "WorldContextObject"))
	static UE_API void SetScalabilityQualityToLow(UObject* WorldContextObject);

	/** Sets all viewports of the first found level editor to have the given ViewMode (Lit/Unlit/etc.) **/
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void SetEditorViewportViewMode(EViewModeIndex Index);

	/** Sets ViewMode (Lit/Unlit/etc.) of the first active viewport found **/
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void SetEditorActiveViewportViewMode(EViewModeIndex Index);

	/** Get the ViewMode (Lit/Unlit/etc.) of the active viewport. Returns VMI_Unknown if fails to get the active viewport view mode index. **/
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API EViewModeIndex GetEditorActiveViewportViewMode();

	/** Sets the opacity for wireframe viewport view modes of the active viewport. **/
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void SetEditorActiveViewportWireframeOpacity(float Opacity);

	/** Retrieves the opacity for wireframe viewport view modes of the active viewport. **/
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API float GetEditorActiveViewportWireframeOpacity();

	/** Sets all viewports of the first found level editor to have the VisualizeBuffer ViewMode and also display a given buffer (BaseColor/Metallic/Roughness/etc.) **/
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void SetEditorViewportVisualizeBuffer(FName BufferName);

	/**
	 * Add info to currently running automated test.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void AddTestInfo(const FString& InLogItem);

	/**
	 * Add warning to currently running automated test.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void AddTestWarning(const FString& InLogItem);

	/**
	 * Add error to currently running automated test.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void AddTestError(const FString& InLogItem);
};

#if WITH_AUTOMATION_TESTS

template<typename T>
class FConsoleVariableSwapperTempl
{
public:
	FConsoleVariableSwapperTempl(FString InConsoleVariableName);

	void Set(T Value);

	void Restore();

private:
	bool bModified;
	FString ConsoleVariableName;

	T OriginalValue;
};

class FAutomationTestScreenshotEnvSetup
{
public:
	FAutomationTestScreenshotEnvSetup();
	~FAutomationTestScreenshotEnvSetup();

	// Disable AA, auto-exposure, motion blur, contact shadow if InOutOptions.bDisableNoisyRenderingFeatures.
	// Update screenshot comparison tolerance stored in InOutOptions.
	// Set visualization buffer name if required.
	void Setup(UWorld* InWorld, FAutomationScreenshotOptions& InOutOptions);

	/** Restore the old settings. */
	void Restore();

private:
	FConsoleVariableSwapperTempl<int32> DefaultFeature_AntiAliasing;
	FConsoleVariableSwapperTempl<int32> DefaultFeature_AutoExposure;
	FConsoleVariableSwapperTempl<int32> DefaultFeature_MotionBlur;
	FConsoleVariableSwapperTempl<int32> MotionBlurQuality;
	FConsoleVariableSwapperTempl<int32> ScreenSpaceReflectionQuality;
	FConsoleVariableSwapperTempl<int32> EyeAdaptationQuality;
	FConsoleVariableSwapperTempl<int32> ContactShadows;
	FConsoleVariableSwapperTempl<float> TonemapperGamma;
	FConsoleVariableSwapperTempl<float> TonemapperSharpen;
	FConsoleVariableSwapperTempl<float> ScreenPercentage;
	FConsoleVariableSwapperTempl<int32> DynamicResTestScreenPercentage;
	FConsoleVariableSwapperTempl<int32> DynamicResOperationMode;
	FConsoleVariableSwapperTempl<float> SecondaryScreenPercentage;

	TWeakObjectPtr<UWorld> WorldPtr;
	TSharedPtr< class FAutomationViewExtension, ESPMode::ThreadSafe > AutomationViewExtension;
};

#endif

#undef UE_API
