// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AutomationViewSettings.h"
#include "AutomationScreenshotOptions.generated.h"

UENUM(BlueprintType)
enum class EComparisonTolerance : uint8
{
	Zero,
	Low,
	Medium,
	High,
	Custom
};

USTRUCT(BlueprintType)
struct FComparisonToleranceAmount
{
	GENERATED_BODY()

public:

	FComparisonToleranceAmount()
		: Red(0)
		, Green(0)
		, Blue(0)
		, Alpha(0)
		, MinBrightness(0)
		, MaxBrightness(255)
	{
	}

	FComparisonToleranceAmount(uint8 R, uint8 G, uint8 B, uint8 A, uint8 InMinBrightness, uint8 InMaxBrightness)
		: Red(R)
		, Green(G)
		, Blue(B)
		, Alpha(A)
		, MinBrightness(InMinBrightness)
		, MaxBrightness(InMaxBrightness)
	{
	}

public:
	UPROPERTY(EditAnywhere, Category="Tolerance")
	uint8 Red;
	
	UPROPERTY(EditAnywhere, Category="Tolerance")
	uint8 Green;
	
	UPROPERTY(EditAnywhere, Category="Tolerance")
	uint8 Blue;
	
	UPROPERTY(EditAnywhere, Category="Tolerance")
	uint8 Alpha;
	
	UPROPERTY(EditAnywhere, Category="Tolerance")
	uint8 MinBrightness;

	UPROPERTY(EditAnywhere, Category="Tolerance")
	uint8 MaxBrightness;
};

USTRUCT(BlueprintType)
struct FAutomationScreenshotOptions
{
	GENERATED_BODY()

public:
	FAutomationScreenshotOptions()
		: Resolution(ForceInit)
		, Delay(0.2f)
		, FrameDelay(5)
		, bOverride_OverrideTimeTo(false)
		, OverrideTimeTo(0.0f)
		, bDisableNoisyRenderingFeatures(true)
		, bDisableTonemapping(true)
		, ViewSettings(nullptr)
		, VisualizeBuffer(NAME_None)
		, Tolerance(EComparisonTolerance::Zero)
		, ToleranceAmount()
		, MaximumLocalError(0.10f)
		, MaximumGlobalError(0.02f)
		, bIgnoreAntiAliasing(true)
		, bIgnoreColors(false)
		, bAllowEmulatingSplitscreen(true)
	{
	}

	FAutomationScreenshotOptions(EComparisonTolerance InTolerance)
		: Resolution(ForceInit)
		, Delay(0.2f)
		, FrameDelay(5)
		, bOverride_OverrideTimeTo(false)
		, OverrideTimeTo(0.0f)
		, bDisableNoisyRenderingFeatures(true)
		, bDisableTonemapping(true)
		, ViewSettings(nullptr)
		, VisualizeBuffer(NAME_None)
		, Tolerance(InTolerance)
		, ToleranceAmount()
		, MaximumLocalError(0.10f)
		, MaximumGlobalError(0.02f)
		, bIgnoreAntiAliasing(true)
		, bIgnoreColors(false)
		, bAllowEmulatingSplitscreen(true)
	{
		SetToleranceAmounts(InTolerance);
	}

	/**
	 * The desired resolution of the screenshot, if none is provided, it will use the default for the
	 * platform setup in the automation settings.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Screenshot")
	FVector2D Resolution;

	/**
	 * The delay before we take the screenshot (measured in seconds). Both this delay and the frame delay must be met before the screenshot is taken.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Screenshot")
	float Delay;

	/**
	 * The delay before we take the screenshot (measured in number of frames). Both this frame delay and the time delay must be met before the screenshot is taken.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Screenshot")
	int32 FrameDelay;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool bOverride_OverrideTimeTo;

	/**
	 * Overrides World Time, Real Time to the value provided.  Sets Delta Time to 0.  Only
	 * affects the time being sent to the render thread and materials.  The time accumulating
	 * on the game thread is unaffected.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Screenshot", meta=(editcondition = "bOverride_OverrideTimeTo"))
	double OverrideTimeTo;

	/**
	 * Disables Anti-Aliasing, Motion Blur, Screen Space Reflections, Eye Adaptation, Tonemapper and Contact
	 * Shadows, because those features contribute a lot to the noise in the final rendered image.  If you're
	 * explicitly looking for changes. Unchecking the option will make accessible the Disable Eye Adaptation
	 * checkbox.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Screenshot")
	bool bDisableNoisyRenderingFeatures;

	/**
	 * Disables Eye Adaptation and sets Tonemapper to fixed gamma curve. Should generally be on unless
	 * testing tone mapping or other post-processing results.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Screenshot", meta = (EditCondition = "!bDisableNoisyRenderingFeatures", DisplayName = "Disable Eye Adaptation"))
	bool bDisableTonemapping;

	/**
	 * Assign custom view settings to control which rendering options we allow on while taking the
	 * screenshot.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Screenshot")
	TObjectPtr<UAutomationViewSettings> ViewSettings;

	/**
	 * Allows you to screenshot a buffer other than the default final lit scene image.  Useful if you're
	 * trying to build a test for a specific GBuffer, that may be harder to tell if errors are introduced
	 * in it.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Screenshot")
	FName VisualizeBuffer;

	/**
	 * These are quick defaults for tolerance levels, we default to low, because generally there's some
	 * constant variability in every pixel's color introduced by TxAA.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Comparison")
	EComparisonTolerance Tolerance;

	/**
	 * For each channel and brightness levels you can control a region where the colors are found to be
	 * essentially the same.  Generally this is necessary as modern rendering techniques tend to introduce
	 * noise constantly to hide aliasing.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Comparison")
	FComparisonToleranceAmount ToleranceAmount;

	/**
	 * After you've accounted for color tolerance changes, you now need to control for local acceptable error.
	 * Which depending on how pixels were colored on triangle edges may be a few percent of the image being
	 * outside the tolerance levels.  Unlike the MaximumGlobalError, the MaximumLocalError works by focusing
	 * on a smaller subset of the image.  These chunks will have be compared to the local error, in an attempt
	 * to locate hot spots of change that are important, that would be ignored by the global error.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Comparison", meta=( ClampMin = 0, ClampMax = 1, UIMin = 0, UIMax = 1 ))
	float MaximumLocalError;

	/**
	 * After you've accounted for color tolerance changes, you now need to control for total acceptable error.
	 * Which depending on how pixels were colored on triangle edges may be a few percent of the image being
	 * outside the tolerance levels.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Comparison", meta=( ClampMin = 0, ClampMax = 1, UIMin = 0, UIMax = 1 ))
	float MaximumGlobalError;

	/**
	 * If this is true, we search neighboring pixels looking for the expected pixel as what may have happened, is
	 * that the pixel shifted a little.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Comparison")
	bool bIgnoreAntiAliasing;

	/**
	 * If this is true, all we compare is luminance of the scene.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Comparison", AdvancedDisplay)
	bool bIgnoreColors;

	/**
	 * If this is true, this test can use emulated splitscreen rendering when r.ScreenshotTest.EmulateSplitscreenMode = 1, 2, or 3.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Comparison")
	bool bAllowEmulatingSplitscreen;

public:
	void SetToleranceAmounts(EComparisonTolerance InTolerance)
	{
		switch ( InTolerance )
		{
		case EComparisonTolerance::Zero:
			ToleranceAmount = FComparisonToleranceAmount(0, 0, 0, 0, 0, 255);
			break;
		case EComparisonTolerance::Low:
			ToleranceAmount = FComparisonToleranceAmount(16, 16, 16, 16, 16, 240);
			break;
		case EComparisonTolerance::Medium:
			ToleranceAmount = FComparisonToleranceAmount(24, 24, 24, 24, 24, 220);
			break;
		case EComparisonTolerance::High:
			ToleranceAmount = FComparisonToleranceAmount(32, 32, 32, 32, 64, 96);
			break;
		}
	}
};
