// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DataAsset.h"

#include "AutomationViewSettings.generated.h"

UCLASS(BlueprintType)
class UAutomationViewSettings : public UDataAsset
{
	GENERATED_BODY()
	
public:
	UAutomationViewSettings()
		: AntiAliasing(true)
		, MotionBlur(true)
		, TemporalAA(true)
		, ScreenSpaceReflections(true)
		, ScreenSpaceAO(true)
		, DistanceFieldAO(true)
		, ContactShadows(true)
		, EyeAdaptation(true)
		, Bloom(true)
	{
	}

	UPROPERTY(EditAnywhere, Category="Rendering")
	bool AntiAliasing;
	
	UPROPERTY(EditAnywhere, Category="Rendering")
	bool MotionBlur;
	
	UPROPERTY(EditAnywhere, Category="Rendering")
	bool TemporalAA;
	
	UPROPERTY(EditAnywhere, Category="Rendering")
	bool ScreenSpaceReflections;
	
	UPROPERTY(EditAnywhere, Category="Rendering")
	bool ScreenSpaceAO;
	
	UPROPERTY(EditAnywhere, Category="Rendering")
	bool DistanceFieldAO;
	
	UPROPERTY(EditAnywhere, Category="Rendering")
	bool ContactShadows;
	
	UPROPERTY(EditAnywhere, Category="Rendering")
	bool EyeAdaptation;

	UPROPERTY(EditAnywhere, Category = "Rendering")
	bool Bloom;
};
