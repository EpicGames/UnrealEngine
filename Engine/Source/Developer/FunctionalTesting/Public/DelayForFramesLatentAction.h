// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "LatentActions.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

struct FLatentActionInfo;

class FDelayForFramesLatentAction : public FPendingLatentAction
{
public:
	FDelayForFramesLatentAction(const FLatentActionInfo& LatentInfo, int32 NumFrames);
	virtual ~FDelayForFramesLatentAction();

	virtual void UpdateOperation(FLatentResponse& Response) override;

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override;
#endif

private:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	int32 FramesRemaining;
};
