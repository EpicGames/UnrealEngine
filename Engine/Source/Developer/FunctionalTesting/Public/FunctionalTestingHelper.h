// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"

/* Wait for all running functional tests to finish */
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(FUNCTIONALTESTING_API, FWaitForFTestsToFinish);

/* Trigger all functional tests on map */
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(FUNCTIONALTESTING_API, FTriggerFTests);

/* Trigger specific functional test on map */
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FUNCTIONALTESTING_API, FTriggerFTest, FString, TestName);

/* Start all functional tests on map and wait for them to finish */
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND(FUNCTIONALTESTING_API, FStartFTestsOnMap);

/* Start specific functional test on map and wait for it to finish */
DEFINE_EXPORTED_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FUNCTIONALTESTING_API, FStartFTestOnMap, FString, TestName);
