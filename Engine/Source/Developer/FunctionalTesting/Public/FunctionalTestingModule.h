// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "FunctionalTest.h"

FUNCTIONALTESTING_API DECLARE_LOG_CATEGORY_EXTERN(LogFunctionalTest, Log, All);

class IFunctionalTestingModule : public IModuleInterface
{
public:
	/** Triggers in sequence all functional tests found on the level */
	virtual void RunAllTestsOnMap(bool bClearLog, bool bRunLooped) = 0;

	/** Runs a single test */
	virtual void RunTestOnMap(const FString& TestName, bool bClearLog, bool bRunLooped) = 0;

	/** Sets that a test is being started */
	virtual void MarkPendingActivation() = 0;

	/** True if a test is about to start */
	virtual bool IsActivationPending() const = 0;

	/** True if there is an active UFunctionalTestingManager */
	virtual bool IsRunning() const = 0;

	/** True if a UFunctionalTestingManager was spawned and is now done */
	virtual bool IsFinished() const = 0;

	/** Sets the active testing manager */
	virtual void SetManager(class UFunctionalTestingManager* NewManager) = 0;

	/** Gets the active testing manager */
	virtual class UFunctionalTestingManager* GetCurrentManager() = 0;

	/** If true, will run tests forever */
	virtual void SetLooping(const bool bLoop) = 0;

	/** Gets a list of maps/tests in the current project */
	UE_DEPRECATED(5.5, "GetMapTests() with four parameters has been deprecated.")
	virtual void GetMapTests(bool bEditorOnlyTests, TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands, TArray<FString>& OutTestMapAssets) const = 0;

	/**
	 * Gets a list of tests and maps in the current project
	 * @param bEditorOnlyTests	Only collect tests which run in the Editor
	 * @param OutTestInfo		Updated to contain execution information for each test
	 * @param OutTestMapAssets	Updated with the unique *.umap assets which contain tests
	 */
	virtual void GetMapTests(bool bEditorOnlyTests, TArray<FFunctionalTestInfo>& OutTestInfo, TArray<FString>& OutTestMapAssets) const = 0;

	/** Gets the debugger singleton or returns NULL */
	static IFunctionalTestingModule& Get()
	{
		static const FName FunctionalTesting(TEXT("FunctionalTesting"));
		return FModuleManager::Get().LoadModuleChecked<IFunctionalTestingModule>(FunctionalTesting);
	}
};
