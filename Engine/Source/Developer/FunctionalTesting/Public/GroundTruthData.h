// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "GroundTruthData.generated.h"

#define UE_API FUNCTIONALTESTING_API

struct FFrame;

/**
 * 
 */
UCLASS(MinimalAPI, BlueprintType)
class UGroundTruthData : public UObject
{
	GENERATED_BODY()

public:
	UE_API UGroundTruthData();

	UFUNCTION(BlueprintCallable, Category = "Automation")
	UE_API void SaveObject(UObject* GroundTruth);

	UFUNCTION(BlueprintCallable, Category = "Automation")
	UE_API UObject* LoadObject();

	UFUNCTION(BlueprintCallable, Category = "Automation")
	UE_API bool CanModify() const;

	UFUNCTION(BlueprintCallable, Category = "Automation")
	UE_API void ResetObject();

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:

	UPROPERTY(EditAnywhere, Category = Data)
	bool bResetGroundTruth;

protected:
	
	UPROPERTY(VisibleAnywhere, Export, Category=Data)
	TObjectPtr<UObject> ObjectData;
};

#undef UE_API
