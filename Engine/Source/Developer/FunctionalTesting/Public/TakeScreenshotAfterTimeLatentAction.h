// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "AutomationScreenshotOptions.h"
#include "LatentActions.h"

struct FLatentActionInfo;

class FTakeScreenshotAfterTimeLatentAction : public FPendingLatentAction
{
public:
	FTakeScreenshotAfterTimeLatentAction(const FLatentActionInfo& LatentInfo, const FString& InScreenshotName, const FString& InNotes, FAutomationScreenshotOptions InOptions);
	virtual ~FTakeScreenshotAfterTimeLatentAction();

	virtual void UpdateOperation(FLatentResponse& Response) override;

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override;
#endif

private:
	void OnScreenshotTakenAndCompared();

private:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	FString ScreenshotName;
	FString Notes;
	float SecondsRemaining;
	bool FinishedLoading;
	bool IssuedScreenshotCapture;
	bool TakenScreenshot;
	FAutomationScreenshotOptions Options;
};

class FWaitForScreenshotComparisonLatentAction : public FPendingLatentAction
{
public:
	FWaitForScreenshotComparisonLatentAction(const FLatentActionInfo& LatentInfo);
	virtual ~FWaitForScreenshotComparisonLatentAction();

	virtual void UpdateOperation(FLatentResponse& Response) override;

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override;
#endif

private:
	void OnScreenshotTakenAndCompared();

private:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	bool TakenScreenshot;
};
